// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MediaHardwareVideoDecoding.usf: Shaders to convert various media decoder outputs to RGBA
 =============================================================================*/
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "/Engine/Private/ColorUtils.ush"

static const float2 kQuadPos[6] = 
{
	float2(-1.0, 1.0),  
	float2( 1.0, 1.0),  
	float2( 1.0,-1.0),
	float2(-1.0,-1.0),
	float2( 1.0,-1.0),
	float2(-1.0, 1.0),
};

static const float2 kQuadUVs[6] = 
{
	float2(0.0, 0.0),  
	float2(1.0, 0.0),  
	float2(1.0, 1.0),
	float2(0.0, 1.0),
	float2(1.0, 1.0),
	float2(0.0, 0.0),
};

static const float4 kYCoCgOffsets = float4(-0.50196078431373, -0.50196078431373, 0, 0);

Texture2D TextureY;
Texture2D TextureUV;
SamplerState PointClampedSamplerY;
SamplerState BilinearClampedSamplerUV;
SamplerState BilinearClampedSamplerUVAlpha;
float4x4 ColorTransform;
uint SrgbToLinear;

void MainVS(in uint GlobalVertexId : SV_VertexID
		  , out float2 TexCoord : TEXCOORD0
		  , out float4 OutPosition : SV_POSITION)
{
	// Output vertex position.
	OutPosition = float4(kQuadPos[GlobalVertexId], 0, 1);

	// Output top left originated UV of the vertex.
	TexCoord = kQuadUVs[GlobalVertexId].xy;
}

void NV12ConvertPS(in noperspective float2 InUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	float3 YUV;
	YUV.x = TextureY.Sample(PointClampedSamplerY, InUV).x;
	YUV.yz = TextureUV.Sample(BilinearClampedSamplerUV, InUV).xy;
	OutColor.xyz = YuvToRgb(YUV, ColorTransform, SrgbToLinear);
	// no alpha support
	OutColor.w = 1.0;
}


void PassThroughPS(in noperspective float2 InUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	OutColor = TextureY.Sample(BilinearClampedSamplerUV, InUV);
}

void Y416ConvertPS(in noperspective float2 InUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	float4 AVYU = TextureY.Sample(BilinearClampedSamplerUV, InUV);

	OutColor.rgb = YuvToRgb(AVYU.yzw, ColorTransform, SrgbToLinear);
	OutColor.a = AVYU.x;
}

void YCoCgConvertPS(in noperspective float2 InUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	float4 CoCgSY = TextureY.Sample(BilinearClampedSamplerUV, InUV);

	CoCgSY += kYCoCgOffsets;
	
	float Scale = (CoCgSY.z * (255.0 / 8.0)) + 1.0;
	
	float Co = CoCgSY.x / Scale;
	float Cg = CoCgSY.y / Scale;
	float Y = CoCgSY.w;
	
	OutColor = float4(Y + Co - Cg, Y + Cg, Y - Co - Cg, 1.0);
	OutColor.xyz = sRGBToLinear(OutColor.xyz);
}

void YCoCgAlphaConvertPS(in noperspective float2 InUV : TEXCOORD0
		  , in float4 SvPosition : SV_POSITION
		  , out float4 OutColor : SV_Target0)
{
	float4 CoCgSY = TextureY.Sample(BilinearClampedSamplerUV, InUV);
	float Alpha = TextureUV.Sample(BilinearClampedSamplerUVAlpha, InUV).x;

	CoCgSY += kYCoCgOffsets;
	
	float Scale = (CoCgSY.z * (255.0 / 8.0)) + 1.0;
	
	float Co = CoCgSY.x / Scale;
	float Cg = CoCgSY.y / Scale;
	float Y = CoCgSY.w;
	
	OutColor = float4(Y + Co - Cg, Y + Cg, Y - Co - Cg, Alpha);
	OutColor.xyz = sRGBToLinear(OutColor.xyz);
}

