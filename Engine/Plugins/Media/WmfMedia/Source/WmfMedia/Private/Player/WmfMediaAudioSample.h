// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "IMediaAudioSample.h"
#include "MediaObjectPool.h"
#include "MediaSampleQueue.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"


/**
 * Implements a media audio sample for WmfMedia.
 */
class FWmfMediaAudioSample
	: public IMediaAudioSample
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	FWmfMediaAudioSample()
		: Channels(0)
		, Duration(FTimespan::Zero())
		, SampleRate(0)
		, Time(FTimespan::Zero(), 0)
	{ }

	/** Virtual destructor. */
	virtual ~FWmfMediaAudioSample() = default;

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InBuffer The sample's data buffer.
	 * @param InSize The size of the sample buffer (in bytes).
	 * @param InTime The sample time (relative to presentation clock).
	 * @param InDuration The duration for which the sample is valid.
	 */
	bool Initialize(
		const uint8* InBuffer,
		uint32 InSize,
		uint32 InChannels,
		uint32 InSampleRate,
		FMediaTimeStamp InTime,
		FTimespan InDuration)
	{
		if ((InBuffer == nullptr) || (InSize == 0))
		{
			return false;
		}

		Buffer.Reset(InSize);
		Buffer.Append(InBuffer, InSize);

		Channels = InChannels;
		Duration = InDuration;
		SampleRate = InSampleRate;
		Time = InTime;

		return true;
	}

public:

	//~ IMediaAudioSample interface

	virtual const void* GetBuffer() override
	{
		return Buffer.GetData();
	}

	virtual uint32 GetChannels() const override
	{
		return Channels;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaAudioSampleFormat GetFormat() const override
	{
		return EMediaAudioSampleFormat::Int16;
	}

	virtual uint32 GetFrames() const override
	{
		return Buffer.Num() / (Channels * sizeof(int16));
	}

	virtual uint32 GetSampleRate() const override
	{
		return SampleRate;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return Time;
	}

private:

	/** The sample's data buffer. */
	TArray<uint8> Buffer;

	/** Number of audio channels. */
	uint32 Channels;

	/** The duration for which the sample is valid. */
	FTimespan Duration;

	/** Audio sample rate (in samples per second). */
	uint32 SampleRate;

	/** Presentation time for which the sample was generated. */
	FMediaTimeStamp Time;
};


/** Implements a pool for WMF audio sample objects. */
class FWmfMediaAudioSamplePool : public TMediaObjectPool<FWmfMediaAudioSample> { };
