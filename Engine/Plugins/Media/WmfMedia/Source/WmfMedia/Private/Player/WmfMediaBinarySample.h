// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "IMediaBinarySample.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"


/**
 * Implements a media binary data sample for WmfMedia.
 */
class FWmfMediaBinarySample
	: public IMediaBinarySample
{
public:

	/** Default constructor. */
	FWmfMediaBinarySample()
		: Duration(FTimespan::Zero())
		, Time(FTimespan::Zero())
	{ }

	/** Virtual destructor. */
	virtual ~FWmfMediaBinarySample() = default;

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InBuffer The sample's data buffer.
	 * @param InSize Size of the buffer.
	 * @param InTime The sample time (relative to presentation clock).
	 * @param InDuration The duration for which the sample is valid.
	 */
	bool Initialize(
		const void* InBuffer,
		uint32 InSize,
		FMediaTimeStamp InTime,
		FTimespan InDuration)
	{
		if ((InBuffer == nullptr) || (InSize == 0))
		{
			return false;
		}

		Buffer.Reset(InSize);
		Buffer.Append((uint8*)InBuffer, InSize);

		Duration = InDuration;
		Time = InTime;

		return true;
	}

public:

	//~ IMediaBinarySample interface

	virtual const void* GetData() override
	{
		return Buffer.GetData();
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual uint32 GetSize() const override
	{
		return Buffer.Num();
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return Time;
	}

private:

	/** The sample's data buffer. */
	TArray<uint8> Buffer;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Presentation time for which the sample was generated. */
	FMediaTimeStamp Time;
};
