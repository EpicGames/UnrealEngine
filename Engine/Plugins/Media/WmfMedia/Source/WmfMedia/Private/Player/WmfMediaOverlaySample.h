// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMediaOverlaySample.h"
#include "Internationalization/Regex.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"


/**
 * Implements an overlay text sample for WmfMedia.
 */
class FWmfMediaOverlaySample
	: public IMediaOverlaySample
{
public:

	/** Default constructor. */
	FWmfMediaOverlaySample()
		: Duration(FTimespan::Zero())
		, Time(FTimespan::Zero())
	{ }

	/** Virtual destructor. */
	virtual ~FWmfMediaOverlaySample() = default;

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InBuffer The sample's data buffer.
	 * @param InTime The sample time (relative to presentation clock).
	 * @param InDuration The duration for which the sample is valid.
	 */
	bool Initialize(
		const char* InBuffer,
		FMediaTimeStamp InTime,
		FTimespan InDuration)
	{
		if (InBuffer == nullptr)
		{
			return false;
		}

		// simply strip all formatting for now
		FString StrippedText;
		{
			static const FRegexPattern StripHtmlPattern(TEXT("<(?:[^>=]|='[^']*'|=\"[^\"]*\"|=[^'\"][^\\s>]*)*>"));

			const FString InputText = ANSI_TO_TCHAR(InBuffer);
			FRegexMatcher Matcher(StripHtmlPattern, InputText);

			int32 TextBegin = INDEX_NONE;

			while (Matcher.FindNext())
			{
				if (TextBegin != INDEX_NONE)
				{
					StrippedText += InputText.Mid(TextBegin, Matcher.GetMatchBeginning() - TextBegin);
				}

				TextBegin = Matcher.GetMatchEnding();
			}

			if (TextBegin > INDEX_NONE)
			{
				StrippedText += InputText.Mid(TextBegin);
			}
		}

		Duration = (InDuration < FTimespan::Zero()) ? FTimespan::MaxValue() : InDuration;
		Text = FText::FromString(StrippedText);
		Time = InTime;

		return true;
	}

public:

	//~ IMediaOverlaySample interface

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual TOptional<FVector2D> GetPosition() const override
	{
		return TOptional<FVector2D>();
	}

	virtual FText GetText() const override
	{
		return Text;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return Time;
	}

	virtual EMediaOverlaySampleType GetType() const override
	{
		return EMediaOverlaySampleType::Caption;
	}

private:

	/** The duration for which the sample is valid. */
	FTimespan Duration;

	/** The overlay text. */
	FText Text;

	/** Presentation time for which the sample was generated. */
	FMediaTimeStamp Time;
};
