// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaPlayer.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Async/Async.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "MediaPlayerOptions.h"
#include "Misc/Optional.h"
#include "UObject/Class.h"
#include "HardwareInfo.h"

#include "Wmf/WmfMediaSession.h"
#include "WmfMediaSettings.h"
#include "WmfMediaTracks.h"
#include "Wmf/WmfMediaUtils.h"

#include "Windows/AllowWindowsPlatformTypes.h"

/* FWmfVideoPlayer structors
 *****************************************************************************/

FWmfMediaPlayer::FWmfMediaPlayer(IMediaEventSink& InEventSink)
	: Duration(0)
	, EventSink(InEventSink)
	, Session(new FWmfMediaSession)
	, Tracks(MakeShared<FWmfMediaTracks, ESPMode::ThreadSafe>())
{
	check(Session != NULL);
	check(Tracks.IsValid());
	Session->SetTracks(Tracks);
	Tracks->SetSession(Session);
}


FWmfMediaPlayer::~FWmfMediaPlayer()
{
	Close();
}


/* IMediaPlayer interface
 *****************************************************************************/

void FWmfMediaPlayer::Close()
{
	if ((Session->GetState() == EMediaState::Closed))
	{
		return;
	}

	Session->Shutdown();

	// reset player
	Duration = 0;
	MediaUrl = FString();
	Tracks->Shutdown();

	// notify listeners
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
}


IMediaCache& FWmfMediaPlayer::GetCache()
{
	return *this;
}


IMediaControls& FWmfMediaPlayer::GetControls()
{
	return *Session;
}


FString FWmfMediaPlayer::GetInfo() const
{
	return Tracks->GetInfo();
}


FGuid FWmfMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x82f2a4c0, 0x225448c6, 0x853ff0a3, 0x0a2b08b7);
	return PlayerPluginGUID;
}

IMediaSamples& FWmfMediaPlayer::GetSamples()
{
	return *Tracks;
}


FString FWmfMediaPlayer::GetStats() const
{
	FString Result;
	Tracks->AppendStats(Result);

	return Result;
}


IMediaTracks& FWmfMediaPlayer::GetTracks()
{
	return *Tracks;
}


FString FWmfMediaPlayer::GetUrl() const
{
	return MediaUrl;
}


IMediaView& FWmfMediaPlayer::GetView()
{
	return *this;
}


bool FWmfMediaPlayer::Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions)
{
	Close();

	if (Url.IsEmpty())
	{
		return false;
	}

	const bool Precache = (Options != nullptr) ? Options->GetMediaOption("PrecacheFile", false) : false;

	return InitializePlayer(nullptr, Url, Precache, PlayerOptions);
}


bool FWmfMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	return Open(Url, Options, nullptr);
}


bool FWmfMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* /*Options*/)
{
	Close();

	if (Archive->TotalSize() == 0)
	{
		UE_LOGF(LogWmfMedia, Verbose, "Player %p: Cannot open media from archive (archive is empty)", this);
		return false;
	}

	if (OriginalUrl.IsEmpty())
	{
		UE_LOGF(LogWmfMedia, Verbose, "Player %p: Cannot open media from archive (no original URL provided)", this);
		return false;
	}

	return InitializePlayer(Archive, OriginalUrl, false, nullptr);
}

void FWmfMediaPlayer::Tick()
{
	bool MediaSourceChanged = false;
	bool TrackSelectionChanged = false;

	Tracks->GetFlags(MediaSourceChanged, TrackSelectionChanged);

	if (MediaSourceChanged)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	}

	if (TrackSelectionChanged)
	{
		// less than windows 10, seem to be a problem switching stream. The issue is also present when hardware acceleration is enabled.
		// If the session is in the error state, we need to reinitialize it.
		if (!FPlatformMisc::VerifyWindowsVersion(10, 0) /* Anything < Windows 10.0 */ ||
			(GetDefault<UWmfMediaSettings>()->HardwareAcceleratedVideoDecoding && FHardwareInfo::GetHardwareInfo(NAME_RHI) == "D3D11") ||
			GetDefault<UWmfMediaSettings>()->bAreHardwareAcceleratedCodecRegistered ||
			Session->GetState() == EMediaState::Error)
		{
			const auto Settings = GetDefault<UWmfMediaSettings>();
			check(Settings != nullptr);

			Session->Initialize(Settings->LowLatency);
			Tracks->ReInitialize();
		}

		bool bOK = false;
		if (Tracks->IsInitialized())
		{
			auto Topo = Tracks->CreateTopology();

			if (Session->SetTopology(Topo, Tracks->GetDuration()))
			{
				bOK = true;
			}
		}

		if (!bOK)
		{
			Session->Shutdown();
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
		}
	}

	if (MediaSourceChanged || TrackSelectionChanged)
	{
		Tracks->ClearFlags();
	}
}


void FWmfMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	Tick();

	// forward session events
	TArray<EMediaEvent> OutEvents;
	Session->GetEvents(OutEvents);

	for (const auto& Event : OutEvents)
	{
		EventSink.ReceiveMediaEvent(Event);
	}
}


bool FWmfMediaPlayer::FlushOnSeekStarted() const
{
	return true;
}


bool FWmfMediaPlayer::FlushOnSeekCompleted() const
{
	return false;
}


bool FWmfMediaPlayer::GetPlayerFeatureFlag(EFeatureFlag Flag) const
{
	switch (Flag)
	{
		case EFeatureFlag::UsePlaybackTimingV2:
		case EFeatureFlag::PlayerUsesInternalFlushOnSeek:
			return true;
	}
	return IMediaPlayer::GetPlayerFeatureFlag(Flag);
}


/* FWmfMediaPlayer implementation
 *****************************************************************************/

bool FWmfMediaPlayer::InitializePlayer(const TSharedPtr<FArchive, ESPMode::ThreadSafe>& Archive, const FString& Url, bool Precache, const FMediaPlayerOptions* PlayerOptions)
{
	UE_LOGF(LogWmfMedia, Verbose, "Player %llx: Initializing %ls (archive = %ls, precache = %ls)", this, *Url, Archive.IsValid() ? TEXT("yes") : TEXT("no"), Precache ? TEXT("yes") : TEXT("no"));

	const auto Settings = GetDefault<UWmfMediaSettings>();
	check(Settings != nullptr);

	if (!Session->Initialize(Settings->LowLatency))
	{
		return false;
	}

	MediaUrl = Url;

	FMediaPlayerOptions LocalPlayerOptions;
	if (PlayerOptions)
	{
		LocalPlayerOptions = *PlayerOptions;
	}

	// initialize presentation on a separate thread
	const EAsyncExecution Execution = Precache ? EAsyncExecution::Thread : EAsyncExecution::ThreadPool;

	Async(Execution, [Archive, Url, Precache, LocalPlayerOptions, TracksPtr = TWeakPtr<FWmfMediaTracks, ESPMode::ThreadSafe>(Tracks)]()
	{
		TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> PinnedTracks = TracksPtr.Pin();

		if (PinnedTracks.IsValid())
		{
			TComPtr<IMFMediaSource> MediaSource = WmfMedia::ResolveMediaSource(Archive, Url, Precache);

			PinnedTracks->Initialize(MediaSource, Url, &LocalPlayerOptions);
		}
	});

	return true;
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
