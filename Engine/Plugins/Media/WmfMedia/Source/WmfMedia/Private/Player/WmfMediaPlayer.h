// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Containers/UnrealString.h"
#include "IMediaCache.h"
#include "IMediaPlayer.h"
#include "IMediaView.h"
#include "Misc/Timespan.h"

#include "Windows/AllowWindowsPlatformTypes.h"

class FWmfMediaSession;
class FWmfMediaTracks;
class IMediaEventSink;
struct FMediaPlayerOptions;


/**
 * Implements a media player using the Windows Media Foundation framework.
 */
class FWmfMediaPlayer
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaView
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FWmfMediaPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FWmfMediaPlayer();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options, const FMediaPlayerOptions* PlayerOptions) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;

	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
	
	virtual bool FlushOnSeekStarted() const override;
	virtual bool FlushOnSeekCompleted() const override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag flag) const override;

protected:

	/**
	 * Initialize the native AvPlayer instance.
	 *
	 * @param Archive The archive being used as a media source (optional).
	 * @param Url The media URL being opened.
	 * @param Precache Whether to precache media into RAM if InURL is a local file.
	 * @return true on success, false otherwise.
	 */
	bool InitializePlayer(const TSharedPtr<FArchive, ESPMode::ThreadSafe>& Archive, const FString& Url, bool Precache, const FMediaPlayerOptions* PlayerOptions);

private:
	/** Tick the player. */
	void Tick();

	/** The duration of the currently loaded media. */
	FTimespan Duration;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** The URL of the currently opened media. */
	FString MediaUrl;

	/** Asynchronous callback object for the media session. */
	TComPtr<FWmfMediaSession> Session;

	/** Media streams collection. */
	TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> Tracks;
};


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
