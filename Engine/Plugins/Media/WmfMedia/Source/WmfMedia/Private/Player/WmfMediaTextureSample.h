// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "IMediaTextureSample.h"
#include "MediaObjectPool.h"
#include "MediaSampleQueue.h"
#include "Math/IntPoint.h"
#include "Misc/Timespan.h"
#include "WmfMediaCodec/WmfMediaDecoder.h"

/**
 * Texture sample generated by WmfMedia player.
 */
class FWmfMediaTextureSample
	: public IMediaTextureSample
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	FWmfMediaTextureSample()
		: Dim(FIntPoint::ZeroValue)
		, Duration(FTimespan::Zero())
		, OutputDim(FIntPoint::ZeroValue)
		, SampleFormat(EMediaTextureSampleFormat::Undefined)
		, Time(FTimespan::Zero())
		, bIsBufferExternal(false)
		, Stride(0)
	{ }

	/** Virtual destructor. */
	virtual ~FWmfMediaTextureSample() = default;

public:

	/**
	 * Initialize the sample.
	 *
	 * @param InBuffer The sample's data buffer.
	 * @param InSize Size of the buffer.
	 * @param InDim The sample buffer's width and height (in pixels).
	 * @param InOutputDim The sample's output width and height (in pixels).
	 * @param InSampleFormat The sample format.
	 * @param InStride Number of bytes per pixel row.
	 * @param InTime The sample time (relative to presentation clock).
	 * @param InDuration The duration for which the sample is valid.
	 */
	bool Initialize(
		const void* InBuffer,
		uint32 InSize,
		const FIntPoint& InDim,
		const FIntPoint& InOutputDim,
		EMediaTextureSampleFormat InSampleFormat,
		uint32 InStride,
		FMediaTimeStamp InTime,
		FTimespan InDuration)
	{
		if ((InBuffer == nullptr) || (InSampleFormat == EMediaTextureSampleFormat::Undefined) || (InSize == 0) || (InStride == 0))
		{
			return false;
		}

		if ((InStride * InDim.Y) > InSize)
		{
			return false;
		}

		Buffer.Reset(InSize);
		Buffer.Append((uint8*)InBuffer, InSize);

		Duration = InDuration;
		Dim = InDim;
		OutputDim = InOutputDim;
		SampleFormat = InSampleFormat;
		Stride = InStride;
		Time = InTime;

		return true;
	}

	/**
	 * Initialize the sample.
	 * Use this when we are using external buffers.
	 * This will move the data out of InBuffer instead of copying it.
	 *
	 * @param InBuffer The sample's data buffer.
	 * @param InDim The sample buffer's width and height (in pixels).
	 * @param InOutputDim The sample's output width and height (in pixels).
	 * @param InSampleFormat The sample format.
	 * @param InStride Number of bytes per pixel row.
	 * @param InTime The sample time (relative to presentation clock).
	 * @param InDuration The duration for which the sample is valid.
	 */
	bool InitializeExternal(
		TArray<uint8>* InBuffer,
		const FIntPoint& InDim,
		const FIntPoint& InOutputDim,
		EMediaTextureSampleFormat InSampleFormat,
		uint32 InStride,
		FMediaTimeStamp InTime,
		FTimespan InDuration)
	{
		if (InBuffer == nullptr)
		{
			return false;
		}

		uint32 InSize = InBuffer->Num();

		if ((InSampleFormat == EMediaTextureSampleFormat::Undefined) || (InSize == 0) || (InStride == 0))
		{
			return false;
		}

		Buffer = MoveTemp(*InBuffer);

		Duration = InDuration;
		Dim = InDim;
		OutputDim = InOutputDim;
		SampleFormat = InSampleFormat;
		Stride = InStride;
		Time = InTime;
		bIsBufferExternal = true;
		return true;
	}

	/**
	 * See if we are using external buffers.
	 * 
	 * @return True if so.
	 */
	bool IsBufferExternal() const { return bIsBufferExternal; }

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() override
	{
		return Buffer.GetData();
	}

	virtual FIntPoint GetDim() const override
	{
		return Dim;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaTextureSampleFormat GetFormat() const override
	{
		return SampleFormat;
	}

	virtual FIntPoint GetOutputDim() const override
	{
		return OutputDim;
	}

	virtual uint32 GetStride() const override
	{
		return Stride;
	}

#if WITH_ENGINE
	virtual FRHITexture* GetTexture() const override
	{
		return nullptr;
	}
#endif //WITH_ENGINE

	virtual FMediaTimeStamp GetTime() const override
	{
		return Time;
	}

	virtual bool IsCacheable() const override
	{
		return true;
	}

	virtual bool IsOutputSrgb() const override
	{
		return true;
	}

	virtual void ShutdownPoolable() override
	{
		if (bIsBufferExternal)
		{
			WmfMediaDecoder::ReturnExternalBuffer(Buffer);
			bIsBufferExternal = false;
		}
	}

protected:

	/** Width and height of the texture sample. */
	FIntPoint Dim;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Width and height of the output. */
	FIntPoint OutputDim;

	/** The sample format. */
	EMediaTextureSampleFormat SampleFormat;

	/** Presentation for which the sample was generated. */
	FMediaTimeStamp Time;

	/** If true, then Buffer is from WmfMediaDecoder and needs to be returned to it. */
	bool bIsBufferExternal;

private:

	/** The sample's data buffer. */
	TArray<uint8> Buffer;

	/** Number of bytes per pixel row. */
	uint32 Stride;

};


/** Implements a pool for WMF texture samples. */
class FWmfMediaTextureSamplePool : public TMediaObjectPool<FWmfMediaTextureSample> { };
