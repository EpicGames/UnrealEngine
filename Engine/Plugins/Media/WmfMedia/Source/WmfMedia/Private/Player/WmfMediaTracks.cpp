// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaTracks.h"
#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "IHeadMountedDisplayModule.h"
#include "IMediaOptions.h"
#include "IWmfMediaModule.h"
#include "MediaHelpers.h"
#include "MediaSampleQueueDepths.h"
#include "MediaPlayerOptions.h"
#include "Misc/ScopeLock.h"
#include "UObject/Class.h"

#if WITH_ENGINE
	#include "Engine/Engine.h"
#endif

#include "Player/WmfMediaTextureSample.h"

#include "WmfMediaAudioSample.h"
#include "WmfMediaBinarySample.h"
#include "WmfMediaCodec/WmfMediaCodecManager.h"
#include "WmfMediaCodec/WmfMediaDecoder.h"
#include "WmfMediaHardwareVideoDecodingTextureSample.h"
#include "WmfMediaOverlaySample.h"
#include "Wmf/WmfMediaSampler.h"
#include "WmfMediaSettings.h"
#include "Wmf/WmfMediaSink.h"
#include "Wmf/WmfMediaStreamSink.h"
#include "Wmf/WmfMediaSession.h"
#include "Wmf/WmfMediaTopologyLoader.h"
#include "Wmf/WmfMediaUtils.h"

#include "Windows/AllowWindowsPlatformTypes.h"

#define LOCTEXT_NAMESPACE "FWmfMediaTracks"

#define WMFMEDIATRACKS_TRACE_FORMATS 0


/* FWmfMediaTracks structors
 *****************************************************************************/

FWmfMediaTracks::FWmfMediaTracks()
	: AudioSamplePool(new FWmfMediaAudioSamplePool)
	, MediaSourceChanged(false)
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedCaptionTrack(INDEX_NONE)
	, SelectedMetadataTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
	, SelectionChanged(false)
	, bVideoTrackRequestedHardwareAcceleration(false)
	, VideoSamplePool(nullptr)
	, VideoHardwareVideoDecodingSamplePool(nullptr)
	, SessionState(EMediaState::Closed)
	, Session(nullptr)
{}


FWmfMediaTracks::~FWmfMediaTracks()
{
	Shutdown();

	delete AudioSamplePool;
	AudioSamplePool = nullptr;

	delete VideoSamplePool;
	VideoSamplePool = nullptr;
}


/* FWmfMediaTracks interface
 *****************************************************************************/

void FWmfMediaTracks::AppendStats(FString &OutStats) const
{
	FScopeLock Lock(&CriticalSection);

	// audio tracks
	OutStats += TEXT("Audio Tracks\n");
	
	if (AudioTracks.Num() == 0)
	{
		OutStats += TEXT("\tnone\n");
	}
	else
	{
		for (const FTrack& Track : AudioTracks)
		{
			OutStats += FString::Printf(TEXT("\t%s\n"), *Track.DisplayName.ToString());
			OutStats += TEXT("\t\tNot implemented yet");
		}
	}

	// video tracks
	OutStats += TEXT("Video Tracks\n");

	if (VideoTracks.Num() == 0)
	{
		OutStats += TEXT("\tnone\n");
	}
	else
	{
		for (const FTrack& Track : VideoTracks)
		{
			OutStats += FString::Printf(TEXT("\t%s\n"), *Track.DisplayName.ToString());
			OutStats += TEXT("\t\tNot implemented yet");
		}
	}
}


void FWmfMediaTracks::ClearFlags()
{
	FScopeLock Lock(&CriticalSection);

	MediaSourceChanged = false;
	SelectionChanged = false;
}


TComPtr<IMFTopology> FWmfMediaTracks::CreateTopology()
{
	FScopeLock Lock(&CriticalSection);

	// validate streams
	if (MediaSource == NULL)
	{
		return NULL; // nothing to play
	}

	if ((SelectedAudioTrack == INDEX_NONE) &&
		(SelectedCaptionTrack == INDEX_NONE) &&
		(SelectedMetadataTrack == INDEX_NONE) &&
		(SelectedVideoTrack == INDEX_NONE))
	{
		return NULL; // no tracks selected
	}

	// create topology
	TComPtr<IMFTopology> Topology;
	{
		const HRESULT Result = ::MFCreateTopology(&Topology);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to create playback topology: %ls", this, *WmfMedia::ResultToString(Result));
			return NULL;
		}
	}

	// add enabled streams to topology
	bool TracksAdded = false;
	bVideoTrackRequestedHardwareAcceleration = false;

	if (AudioTracks.IsValidIndex(SelectedAudioTrack))
	{
		TracksAdded |= AddTrackToTopology(AudioTracks[SelectedAudioTrack], *Topology);
	}

	if (CaptionTracks.IsValidIndex(SelectedCaptionTrack))
	{
		TracksAdded |= AddTrackToTopology(CaptionTracks[SelectedCaptionTrack], *Topology);
	}

	if (MetadataTracks.IsValidIndex(SelectedMetadataTrack))
	{
		TracksAdded |= AddTrackToTopology(MetadataTracks[SelectedMetadataTrack], *Topology);
	}

	if (VideoTracks.IsValidIndex(SelectedVideoTrack))
	{
		TracksAdded |= AddTrackToTopology(VideoTracks[SelectedVideoTrack], *Topology);
	}

	if (!TracksAdded)
	{
		return NULL;
	}

	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Created playback topology %p (media source %p)", this, Topology.Get(), MediaSource.Get());

	const UWmfMediaSettings* WmfMediaSettings = GetDefault<UWmfMediaSettings>();
	if (WmfMediaSettings->HardwareAcceleratedVideoDecoding ||
		WmfMediaSettings->bAreHardwareAcceleratedCodecRegistered)
	{
		WmfMediaTopologyLoader MediaTopologyLoader;
		bool bHardwareAccelerated = MediaTopologyLoader.EnableHardwareAcceleration(Topology) || bVideoTrackRequestedHardwareAcceleration;

		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Video (media source %p) will be decoded on %ls", this, MediaSource.Get(), bHardwareAccelerated ? TEXT("GPU") : TEXT("CPU"));
		Info += FString::Printf(TEXT("Video decoded on %s\n"), bHardwareAccelerated ? TEXT("GPU") : TEXT("CPU"));
	}

	return Topology;
}


FTimespan FWmfMediaTracks::GetDuration() const
{
	FScopeLock Lock(&CriticalSection);

	if (PresentationDescriptor == NULL)
	{
		return FTimespan::Zero();
	}

	UINT64 PresentationDuration = 0;
	HRESULT Result = PresentationDescriptor->GetUINT64(MF_PD_DURATION, &PresentationDuration);
	if (SUCCEEDED(Result) == false)
	{
		// Live streams like webcam do not have a duration.
		// Return max value to signify this.
		return FTimespan::MaxValue();
	}
	
	// The duration reported here for HAP videos can be larger than they really are be by this amount.
	PresentationDuration -= 10000;

	return FTimespan(PresentationDuration);
}


void FWmfMediaTracks::GetFlags(bool& OutMediaSourceChanged, bool& OutSelectionChanged) const
{
	FScopeLock Lock(&CriticalSection);

	OutMediaSourceChanged = MediaSourceChanged;
	OutSelectionChanged = SelectionChanged;
}


void FWmfMediaTracks::Initialize(IMFMediaSource* InMediaSource, const FString& Url, const FMediaPlayerOptions* PlayerOptions)
{
	Shutdown();

	UE_LOGF(LogWmfMedia, Verbose, "Tracks: %p: Initializing (media source %p)", this, InMediaSource);

	FScopeLock Lock(&CriticalSection);

	MediaSourceChanged = true;
	SelectionChanged = true;

	if (InMediaSource == NULL)
	{
		return;
	}

	// create presentation descriptor
	TComPtr<IMFPresentationDescriptor> NewPresentationDescriptor;
	{
		const HRESULT Result = InMediaSource->CreatePresentationDescriptor(&NewPresentationDescriptor);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to create presentation descriptor: %ls", this, *WmfMedia::ResultToString(Result));
			return;
		}
	}

	// get number of streams
	DWORD StreamCount = 0;
	{
		const HRESULT Result = NewPresentationDescriptor->GetStreamDescriptorCount(&StreamCount);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get stream count: %ls", this, *WmfMedia::ResultToString(Result));
			return;
		}

		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Found %i streams", this, StreamCount);
	}

	// initialization successful
	MediaSource = InMediaSource;
	SourceUrl = Url;

	PresentationDescriptor = NewPresentationDescriptor;
	CachedDuration = GetDuration();

	// add streams (Media Foundation reports them in reverse order)
	bool IsVideoDevice = Url.StartsWith(TEXT("vidcap://"));
	bool AllStreamsAdded = true;

	for (int32 StreamIndex = StreamCount - 1; StreamIndex >= 0; --StreamIndex)
	{
		AllStreamsAdded &= AddStreamToTracks(StreamIndex, IsVideoDevice, Info);
		Info += TEXT("\n");
	}

	if (!AllStreamsAdded)
	{
		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Not all available streams were added to the track collection", this);
	}

	// Tracks must be selected before Session->SetTopology() is called
	FMediaPlayerTrackOptions TrackOptions;
	if (PlayerOptions && PlayerOptions->TrackSelection == EMediaPlayerOptionTrackSelectMode::UseTrackOptionIndices)
	{
		TrackOptions = PlayerOptions->Tracks;
	}

	SelectTrack(EMediaTrackType::Audio, TrackOptions.Audio);
	SelectTrack(EMediaTrackType::Caption, TrackOptions.Caption);
	SelectTrack(EMediaTrackType::Metadata, TrackOptions.Metadata);
	SelectTrack(EMediaTrackType::Video, TrackOptions.Video);
}

void FWmfMediaTracks::ReInitialize()
{
	if (MediaSource != NULL)
	{
		TComPtr<IMFMediaSource> lMediaSource = WmfMedia::ResolveMediaSource(nullptr, SourceUrl, false);

		int32 lTrack = GetSelectedTrack(EMediaTrackType::Video);
		int32 lFormat = GetTrackFormat(EMediaTrackType::Video, lTrack);
		int32 aTrack = GetSelectedTrack(EMediaTrackType::Audio);
		int32 aFormat = GetTrackFormat(EMediaTrackType::Audio, aTrack);
		int32 cTrack = GetSelectedTrack(EMediaTrackType::Caption);
		int32 cFormat = GetTrackFormat(EMediaTrackType::Caption, cTrack);

		Initialize(lMediaSource, SourceUrl, nullptr);
		SetTrackFormat(EMediaTrackType::Video, lTrack, lFormat);
		SelectTrack(EMediaTrackType::Video, lTrack);
		SetTrackFormat(EMediaTrackType::Audio, aTrack, aFormat);
		SelectTrack(EMediaTrackType::Audio, aTrack);
		SetTrackFormat(EMediaTrackType::Caption, cTrack, cFormat);
		SelectTrack(EMediaTrackType::Caption, cTrack);
	}
}

void FWmfMediaTracks::Shutdown()
{
	UE_LOGF(LogWmfMedia, Verbose, "Tracks: %p: Shutting down (media source %p)", this, MediaSource.Get());

	FScopeLock Lock(&CriticalSection);

	AudioSamplePool->Reset();
	if (VideoSamplePool)
	{
		VideoSamplePool->Reset();
	}
	if (VideoHardwareVideoDecodingSamplePool)
	{
		VideoHardwareVideoDecodingSamplePool->Reset();
	}
	MediaStreamSink = nullptr;

	SelectedAudioTrack = INDEX_NONE;
	SelectedCaptionTrack = INDEX_NONE;
	SelectedMetadataTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;

	AudioTracks.Empty();
	MetadataTracks.Empty();
	CaptionTracks.Empty();
	VideoTracks.Empty();

	Info.Empty();

	if (MediaSource != NULL)
	{
		MediaSource->Shutdown();
		MediaSource.Reset();
	}

	PresentationDescriptor.Reset();

	MediaSourceChanged = false;
	SelectionChanged = false;

	SeekIndex = 0;
	LoopIndex = 0;
}

void FWmfMediaTracks::SetSessionState(EMediaState InState)
{
	UE_LOGF(LogWmfMedia, VeryVerbose, "FWmfMediaTracks::SetSessionState %d", int(InState));
	SessionState = InState;
}


void FWmfMediaTracks::SeekStarted(const FTimespan& InTime, int32 InNewSequenceIndex)
{
	UE_LOGF(LogWmfMedia, VeryVerbose, "FWmfMediaTracks::SeekStarted %f", InTime.GetTotalSeconds());

	{
	FScopeLock Lock(&CriticalSection);
	SeekIndex = InNewSequenceIndex;
	LoopIndex = 0;
	}

	if (MediaStreamSink)
	{
		MediaStreamSink->OnSeekStarted();
	}
}

void FWmfMediaTracks::LoopStarted(float SessionRate)
{
	FScopeLock Lock(&CriticalSection);
	{
	if (SessionRate >= 0.0f)
	{
		++LoopIndex;
	}
	else
	{
		--LoopIndex;
	}
	}

	if (MediaStreamSink)
	{
		// We loop, but then: for the sink it's really just a seek
		MediaStreamSink->OnSeekStarted();
	}
}


void FWmfMediaTracks::SessionEnded()
{
	if (MediaStreamSink)
	{
		MediaStreamSink->OnSessionEnded();
	}
}


void FWmfMediaTracks::RequestMoreVideoDataFromStreamSink()
{
	if (MediaStreamSink)
	{
		MediaStreamSink->OnRequestMoreData();
	}
}


bool FWmfMediaTracks::ExecuteOnceMediaStreamSinkHasNoPendingRequests(TFunction<void()>&& ExecuteOnIdle)
{
	if (MediaStreamSink)
	{
		return MediaStreamSink->ExecuteOnceNoPendingRequests(MoveTemp(ExecuteOnIdle));
	}
	return false;
}

/* IMediaSamples interface
 *****************************************************************************/

bool FWmfMediaTracks::FetchAudio(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> Sample;

	if (!AudioSampleQueue.Peek(Sample))
	{
		return false;
	}

	const FMediaTimeStamp SampleTime = Sample->GetTime();

	if (!TimeRange.Overlaps(TRange<FMediaTimeStamp>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	if (!AudioSampleQueue.Dequeue(Sample))
	{
		return false;
	}

	OutSample = Sample;

	return true;
}


bool FWmfMediaTracks::FetchCaption(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe> Sample;

	if (!CaptionSampleQueue.Peek(Sample))
	{
		return false;
	}

	const FMediaTimeStamp SampleTime = Sample->GetTime();

	if (!TimeRange.Overlaps(TRange<FMediaTimeStamp>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	if (!CaptionSampleQueue.Dequeue(Sample))
	{
		return false;
	}

	OutSample = Sample;

	return true;
}


bool FWmfMediaTracks::FetchMetadata(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe> Sample;

	if (!MetadataSampleQueue.Peek(Sample))
	{
		return false;
	}

	const FMediaTimeStamp SampleTime = Sample->GetTime();

	if (!TimeRange.Overlaps(TRange<FMediaTimeStamp>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	if (!MetadataSampleQueue.Dequeue(Sample))
	{
		return false;
	}

	OutSample = Sample;

	return true;
}


bool FWmfMediaTracks::FetchVideo(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
{
	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

	if (!VideoSampleQueue.Peek(Sample))
	{
		return false;
	}

	const FMediaTimeStamp SampleTime = Sample->GetTime();

	if (!TimeRange.Overlaps(TRange<FMediaTimeStamp>(SampleTime, SampleTime + Sample->GetDuration())))
	{
		return false;
	}

	Session->RequestMoreVideoData();

	if (!VideoSampleQueue.Dequeue(Sample))
	{
		return false;
	}

	OutSample = Sample;

	return true;
}


void FWmfMediaTracks::FlushSamples()
{
	Session->EnqueueWorkItem(FWmfMediaSession::FWorkItem([this](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FWmfMediaSession::FBaseWorkItemState* BaseState) -> FWmfMediaSession::EWorkItemResult
		{
			{
			FScopeLock Lock(&CriticalSection);

			UE_LOGF(LogWmfMedia, VeryVerbose, "FWmfMediaTracks::FlushSamples");

			AudioSampleQueue.RequestFlush();
			CaptionSampleQueue.RequestFlush();
			MetadataSampleQueue.RequestFlush();
			VideoSampleQueue.RequestFlush();

			SeekIndex = 0;
			LoopIndex = 0;
			NextMinSequenceIndex.Reset();
			}

			Session->Flush();

			return FWmfMediaSession::EWorkItemResult::Done;
		}));
}


IMediaSamples::EFetchBestSampleResult FWmfMediaTracks::FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult)
{
	EMediaSampleQueueFetchResult FetchRes =	VideoSampleQueue.FetchBestSampleForTimeRange(TimeRange, OutSample, bReverse, bConsistentResult);
	EFetchBestSampleResult Ret = FetchRes == EMediaSampleQueueFetchResult::Found ? EFetchBestSampleResult::Ok :
								 FetchRes == EMediaSampleQueueFetchResult::None ? EFetchBestSampleResult::NoSample : EFetchBestSampleResult::PurgedToEmpty;

	Session->RequestMoreVideoData();

	return Ret;
}

void FWmfMediaTracks::SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex)
{
	FScopeLock Lock(&CriticalSection);
	if (!InNextSequenceIndex.IsSet())
	{
		NextMinSequenceIndex.Reset();
		return;
	}
	if (NextMinSequenceIndex == InNextSequenceIndex)
	{
		return;
	}
	NextMinSequenceIndex = InNextSequenceIndex;
	int32 nsi = InNextSequenceIndex.GetValue();

	AudioSampleQueue.PurgeUntilSequenceIndex(nsi);
	MetadataSampleQueue.PurgeUntilSequenceIndex(nsi);
	CaptionSampleQueue.PurgeUntilSequenceIndex(nsi);
	VideoSampleQueue.PurgeUntilSequenceIndex(nsi);
}


bool FWmfMediaTracks::PeekVideoSampleTime(FMediaTimeStamp & TimeStamp)
{
	TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> Sample;

	bool Ret = VideoSampleQueue.Peek(Sample);

	Session->RequestMoreVideoData();

	if (Ret)
	{
		TimeStamp = Sample->GetTime();
	}
	return Ret;
}

bool FWmfMediaTracks::DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
{
	return VideoSampleQueue.Discard(TimeRange, bReverse);
}

bool FWmfMediaTracks::DiscardAudioSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
{
	return AudioSampleQueue.Discard(TimeRange, bReverse);
}

bool FWmfMediaTracks::DiscardCaptionSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
{
	return CaptionSampleQueue.Discard(TimeRange, bReverse);
}

bool FWmfMediaTracks::DiscardMetadataSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse)
{
	return MetadataSampleQueue.Discard(TimeRange, bReverse);
}

/* IMediaTracks interface
 *****************************************************************************/

bool FWmfMediaTracks::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	FScopeLock Lock(&CriticalSection);

	const FFormat* Format = GetAudioFormat(TrackIndex, FormatIndex);
	
	if (Format == nullptr)
	{
		return false; // format not found
	}

	OutFormat.BitsPerSample = Format->Audio.BitsPerSample;
	OutFormat.NumChannels = Format->Audio.NumChannels;
	OutFormat.SampleRate = Format->Audio.SampleRate;
	OutFormat.TypeName = Format->TypeName;

	return true;
}


int32 FWmfMediaTracks::GetNumTracks(EMediaTrackType TrackType) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.Num();

	case EMediaTrackType::Metadata:
		return MetadataTracks.Num();

	case EMediaTrackType::Caption:
		return CaptionTracks.Num();

	case EMediaTrackType::Video:
		return VideoTracks.Num();

	default:
		break; // unsupported track type
	}

	return 0;
}


int32 FWmfMediaTracks::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Formats.Num();
		}
		break;

	case EMediaTrackType::Metadata:
		if (MetadataTracks.IsValidIndex(TrackIndex))
		{
			return 1;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return 1;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Formats.Num();
		}
		break;

	default:
		break; // unsupported track type
	}

	return 0;
}


int32 FWmfMediaTracks::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return SelectedAudioTrack;

	case EMediaTrackType::Caption:
		return SelectedCaptionTrack;

	case EMediaTrackType::Metadata:
		return SelectedMetadataTrack;

	case EMediaTrackType::Video:
		return SelectedVideoTrack;

	default:
		break; // unsupported track type
	}

	return INDEX_NONE;
}


FText FWmfMediaTracks::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].DisplayName;
		}
		break;
	
	case EMediaTrackType::Metadata:
		if (MetadataTracks.IsValidIndex(TrackIndex))
		{
			return MetadataTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].DisplayName;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].DisplayName;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FText::GetEmpty();
}


int32 FWmfMediaTracks::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	const FTrack* Track = GetTrack(TrackType, TrackIndex);
	return (Track != nullptr) ? Track->SelectedFormat : INDEX_NONE;
}


FString FWmfMediaTracks::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Metadata:
		if (MetadataTracks.IsValidIndex(TrackIndex))
		{
			return MetadataTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Language;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FString();
}


FString FWmfMediaTracks::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FScopeLock Lock(&CriticalSection);

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Metadata:
		if (MetadataTracks.IsValidIndex(TrackIndex))
		{
			return MetadataTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Name;
		}
		break;

	default:
		break; // unsupported track type
	}

	return FString();
}


bool FWmfMediaTracks::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	FScopeLock Lock(&CriticalSection);

	const FFormat* Format = GetVideoFormat(TrackIndex, FormatIndex);
	
	if (Format == nullptr)
	{
		return false; // format not found
	}

	OutFormat.Dim = Format->Video.OutputDim;
	OutFormat.FrameRate = Format->Video.FrameRate;
	OutFormat.FrameRates = Format->Video.FrameRates;
	OutFormat.TypeName = Format->TypeName;

	return true;
}


bool FWmfMediaTracks::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	if (PresentationDescriptor == NULL)
	{
		return false; // not initialized
	}

	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Selecting %ls track %i", this, *MediaUtils::TrackTypeToString(TrackType), TrackIndex);

	FScopeLock Lock(&CriticalSection);

	int32* SelectedTrack = nullptr;
	TArray<FTrack>* Tracks = nullptr;

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		SelectedTrack = &SelectedAudioTrack;
		Tracks = &AudioTracks;
		break;

	case EMediaTrackType::Caption:
		SelectedTrack = &SelectedCaptionTrack;
		Tracks = &CaptionTracks;
		break;

	case EMediaTrackType::Metadata:
		SelectedTrack = &SelectedMetadataTrack;
		Tracks = &MetadataTracks;
		break;

	case EMediaTrackType::Video:
		SelectedTrack = &SelectedVideoTrack;
		Tracks = &VideoTracks;
		break;

	default:
		return false; // unsupported track type
	}

	check(SelectedTrack != nullptr);
	check(Tracks != nullptr);

	if (TrackIndex == *SelectedTrack)
	{
		return true; // already selected
	}

	if ((TrackIndex != INDEX_NONE) && !Tracks->IsValidIndex(TrackIndex))
	{
		return false; // invalid track
	}

	// deselect stream for old track
	if (*SelectedTrack != INDEX_NONE)
	{
		const DWORD StreamIndex = (*Tracks)[*SelectedTrack].StreamIndex;
		const HRESULT Result = PresentationDescriptor->DeselectStream(StreamIndex);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to deselect stream %i on presentation descriptor: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
			return false;
		}

		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Disabled stream %i", this, StreamIndex);
			
		*SelectedTrack = INDEX_NONE;
		SelectionChanged = true;
	}

	// select stream for new track
	if (TrackIndex != INDEX_NONE)
	{
		const DWORD StreamIndex = (*Tracks)[TrackIndex].StreamIndex;
		const HRESULT Result = PresentationDescriptor->SelectStream(StreamIndex);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to enable %ls track %i (stream %i): %ls", this, *MediaUtils::TrackTypeToString(TrackType), TrackIndex, StreamIndex, *WmfMedia::ResultToString(Result));
			return false;
		}

		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Enabled stream %i", this, StreamIndex);

		*SelectedTrack = TrackIndex;
		SelectionChanged = true;
	}

	return true;
}


bool FWmfMediaTracks::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Setting format on %ls track %i to %i", this, *MediaUtils::TrackTypeToString(TrackType), TrackIndex, FormatIndex);

	FScopeLock Lock(&CriticalSection);

	TArray<FTrack>* Tracks = nullptr;

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		Tracks = &AudioTracks;
		break;

	case EMediaTrackType::Caption:
		Tracks = &CaptionTracks;
		break;

	case EMediaTrackType::Metadata:
		Tracks = &MetadataTracks;
		break;

	case EMediaTrackType::Video:
		Tracks = &VideoTracks;
		break;

	default:
		return false; // unsupported track type
	};

	check(Tracks != nullptr);

	if (!Tracks->IsValidIndex(TrackIndex))
	{
		return false; // invalid track index
	}

	FTrack& Track = (*Tracks)[TrackIndex];

	if (Track.SelectedFormat == FormatIndex)
	{
		return true; // format already set
	}

	if (!Track.Formats.IsValidIndex(FormatIndex))
	{
		return false; // invalid format index
	}

	// set track format
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Set format %i instead of %i on %ls track %i (%i formats)", this, FormatIndex, Track.SelectedFormat, *MediaUtils::TrackTypeToString(TrackType), TrackIndex, Track.Formats.Num());

	Track.SelectedFormat = FormatIndex;
	SelectionChanged = true;

	return true;
}


bool FWmfMediaTracks::SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate)
{
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Setting frame rate on format %i of video track %i to %f", this, FormatIndex, TrackIndex, FrameRate);

	FScopeLock Lock(&CriticalSection);

	const FFormat* Format = GetVideoFormat(TrackIndex, FormatIndex);

	if (Format == nullptr)
	{
		return false; // format not found
	}

	if (Format->Video.FrameRate == FrameRate)
	{
		return true; // frame rate already set
	}

	if (!Format->Video.FrameRates.Contains(FrameRate))
	{
		return false; // frame rate not supported
	}

	int32 Numerator;
	int32 Denominator;

	if (!WmfMedia::FrameRateToRatio(FrameRate, Numerator, Denominator))
	{
		return false; // invalid frame rate
	}

	return SUCCEEDED(::MFSetAttributeRatio(Format->InputType, MF_MT_FRAME_RATE, Numerator, Denominator));
}


/* FWmfMediaTracks implementation
 *****************************************************************************/

bool FWmfMediaTracks::AddTrackToTopology(const FTrack& Track, IMFTopology& Topology)
{
	// skip if no format selected
	if (!Track.Formats.IsValidIndex(Track.SelectedFormat))
	{
		return false;
	}

	// get selected format
	const FFormat& Format = Track.Formats[Track.SelectedFormat];

	check(Format.InputType.IsValid());
	check(Format.OutputType.IsValid());

#if WMFMEDIATRACKS_TRACE_FORMATS
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Adding stream %i to topology", this, Track.StreamIndex);
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Input type:\n%ls", this, *WmfMedia::DumpAttributes(*Format.InputType.Get()));
	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Output type:\n%ls", this, *WmfMedia::DumpAttributes(*Format.OutputType.Get()));
#endif

	GUID MajorType;
	{
		const HRESULT Result = Format.OutputType->GetGUID(MF_MT_MAJOR_TYPE, &MajorType);
		check(SUCCEEDED(Result));
	}

	// skip audio if necessary
	if (MajorType == MFMediaType_Audio)
	{
		if (::waveOutGetNumDevs() == 0)
		{
			return false; // no audio device
		}

#if WITH_ENGINE
		if ((GEngine != nullptr) && !GEngine->UseSound())
		{
			return false; // audio disabled
		}

		if ((GEngine == nullptr) && !GetDefault<UWmfMediaSettings>()->NativeAudioOut)
		{
			return false; // no engine audio
		}
#else
		if (!GetDefault<UWmfMediaSettings>()->NativeAudioOut)
		{
			return false; // native audio disabled
		}
#endif
	}

	// set input type
	check(Track.Handler.IsValid());
	{
		const HRESULT Result = Track.Handler->SetCurrentMediaType(Format.InputType);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to set current media type for stream %i: %ls", this, Track.StreamIndex, *WmfMedia::ResultToString(Result));
			return false;
		}
	}

	// create output activator
	TComPtr<IMFActivate> OutputActivator;

	if ((MajorType == MFMediaType_Audio) && GetDefault<UWmfMediaSettings>()->NativeAudioOut)
	{
		// create native audio renderer
		HRESULT Result = MFCreateAudioRendererActivate(&OutputActivator);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to create audio renderer for stream %i: %ls", this, Track.StreamIndex, *WmfMedia::ResultToString(Result));
			return false;
		}

#if WITH_ENGINE
		// allow HMD to override audio output device
		if (IHeadMountedDisplayModule::IsAvailable())
		{
			FString AudioOutputDevice = IHeadMountedDisplayModule::Get().GetAudioOutputDevice();

			if (!AudioOutputDevice.IsEmpty())
			{
				Result = OutputActivator->SetString(MF_AUDIO_RENDERER_ATTRIBUTE_ENDPOINT_ID, *AudioOutputDevice);

				if (FAILED(Result))
				{
					UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to override HMD audio output device for stream %i: %ls", this, Track.StreamIndex, *WmfMedia::ResultToString(Result));
					return false;
				}
			}
		}
#endif //WITH_ENGINE
	}
	else
	{
		// create custom sampler
		TComPtr<FWmfMediaSampler> Sampler = new FWmfMediaSampler();

		if (MajorType == MFMediaType_Audio)
		{
			Sampler->OnClock().AddRaw(this, &FWmfMediaTracks::HandleMediaSamplerClock, EMediaTrackType::Audio);
			Sampler->OnSample().AddRaw(this, &FWmfMediaTracks::HandleMediaSamplerAudioSample);
		}
		else if (MajorType == MFMediaType_SAMI)
		{
			Sampler->OnSample().AddRaw(this, &FWmfMediaTracks::HandleMediaSamplerCaptionSample);
		}
		else if (MajorType == MFMediaType_Binary)
		{
			Sampler->OnSample().AddRaw(this, &FWmfMediaTracks::HandleMediaSamplerMetadataSample);
		}
		else if (MajorType == MFMediaType_Video)
		{
			Sampler->OnSample().AddRaw(this, &FWmfMediaTracks::HandleMediaSamplerVideoSample);
		}

		const HRESULT Result = ::MFCreateSampleGrabberSinkActivate(Format.OutputType, Sampler, &OutputActivator);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to create sampler grabber sink for stream %i: %ls", this, Track.StreamIndex, *WmfMedia::ResultToString(Result));
			return false;
		}
	}

	if (!OutputActivator.IsValid())
	{
		return false;
	}

	// set up output node
	TComPtr<IMFTopologyNode> OutputNode;

	const UWmfMediaSettings* WmfMediaSettings = GetDefault<UWmfMediaSettings>();

	if (VideoSamplePool)
	{
		delete VideoSamplePool;
		VideoSamplePool = nullptr;
	}

#if WITH_ENGINE
	if ((GEngine != nullptr) &&
		(WmfMediaSettings->HardwareAcceleratedVideoDecoding || WmfMediaSettings->bAreHardwareAcceleratedCodecRegistered) &&
		MajorType == MFMediaType_Video &&
		FPlatformMisc::VerifyWindowsVersion(6, 2) && // Windows 8
		FWmfMediaStreamSink::Create(MFMediaType_Video, MediaStreamSink))
	{
		VideoHardwareVideoDecodingSamplePool = MakeShared<FWmfMediaHardwareVideoDecodingTextureSamplePool>();

		MediaStreamSink->SetMediaSamplePoolAndQueue(VideoHardwareVideoDecodingSamplePool, &VideoSampleQueue,
			[this](FTimespan Time, EMediaTrackType TrackType) -> FMediaTimeStamp
			{
				return AdjustTimeStamp(Time, TrackType);
			},
			[this]() -> bool
			{
				return IsSampleStillDesired();
			}
		);

		if (FAILED(::MFCreateTopologyNode(MF_TOPOLOGY_OUTPUT_NODE, &OutputNode)) ||
			FAILED(OutputNode->SetObject((IMFStreamSink*)MediaStreamSink)) ||
			FAILED(OutputNode->SetUINT32(MF_TOPONODE_STREAMID, 0)) ||
			FAILED(OutputNode->SetUINT32(MF_TOPONODE_NOSHUTDOWN_ON_REMOVE, FALSE)) ||
			FAILED(Topology.AddNode(OutputNode)))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to configure output node for stream %i", this, Track.StreamIndex);
			return false;
		}
		bVideoTrackRequestedHardwareAcceleration = true;
	}
	else
#endif
	{
//TODO: >>> SOFTWARE VERSION OR WHAT? THIS IS ALL 100% DIFFERENT?!?!? WTH? (just a "standard" output node takes over) ===========> Sounds like a VERY bad idea!
		VideoSamplePool = new FWmfMediaTextureSamplePool();
		if (!VideoSamplePool)
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to configure output node for stream (no memory) %i", this, Track.StreamIndex);
			return false;
		}

		if (FAILED(::MFCreateTopologyNode(MF_TOPOLOGY_OUTPUT_NODE, &OutputNode)) ||
			FAILED(OutputNode->SetObject(OutputActivator)) ||
			FAILED(OutputNode->SetUINT32(MF_TOPONODE_STREAMID, 0)) ||
			FAILED(Topology.AddNode(OutputNode)))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to configure output node for stream %i", this, Track.StreamIndex);
			return false;
		}
	}

	// set up source node
	TComPtr<IMFTopologyNode> SourceNode;

	if (FAILED(::MFCreateTopologyNode(MF_TOPOLOGY_SOURCESTREAM_NODE, &SourceNode)) ||
		FAILED(SourceNode->SetUnknown(MF_TOPONODE_SOURCE, MediaSource)) ||
		FAILED(SourceNode->SetUnknown(MF_TOPONODE_PRESENTATION_DESCRIPTOR, PresentationDescriptor)) ||
		FAILED(SourceNode->SetUnknown(MF_TOPONODE_STREAM_DESCRIPTOR, Track.Descriptor)) ||
		FAILED(Topology.AddNode(SourceNode)))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to configure source node for stream %i", this, Track.StreamIndex);
		return false;
	}

	// Check subtype
	GUID SubType;

	if (FAILED(Format.InputType->GetGUID(MF_MT_SUBTYPE, &SubType)))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Unable to query MF_MT_SUBTYPE", this);
		return false;
	}

	HRESULT Result = S_OK;

	TComPtr<IMFTopologyNode> DecoderNode;
	TComPtr<IMFTransform> Transform;

	IWmfMediaModule* WmfMediaModule = IWmfMediaModule::Get();

	if (WmfMediaModule && 
		WmfMediaModule->GetCodecManager()->SetupDecoder(MajorType, SubType, DecoderNode, Transform) &&
		DecoderNode &&
		Transform)
	{
		if (FAILED(DecoderNode->SetObject(Transform)) ||
			FAILED(DecoderNode->SetUINT32(MF_TOPONODE_STREAMID, 0)) ||
			FAILED(Topology.AddNode(DecoderNode)) ||
			FAILED(SourceNode->ConnectOutput(0, DecoderNode, 0)) ||
			FAILED(DecoderNode->ConnectOutput(0, OutputNode, 0)) ||
			FAILED(DecoderNode->SetUINT32(MF_TOPONODE_CONNECT_METHOD, MF_CONNECT_ALLOW_CONVERTER)) ||
			FAILED(OutputNode->SetUINT32(MF_TOPONODE_CONNECT_METHOD, MF_CONNECT_ALLOW_CONVERTER)))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to configure decoder node for stream %i", this, Track.StreamIndex);
			return false;
		}

		// Get the decoder.
		WmfMediaDecoder* Decoder = (WmfMediaDecoder*)Transform.Get();
		if (Decoder != nullptr)
		{
			// Enable external buffers.
			if (Decoder->IsExternalBufferSupported())
			{
				Decoder->EnableExternalBuffer(true);

				if (MediaStreamSink.IsValid())
				{
					MediaStreamSink->SetDecoder(Decoder);
				}
			}
		}
	}
	else
	{
		// connect nodes
		Result = SourceNode->ConnectOutput(0, OutputNode, 0);
	}

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to connect topology nodes for stream %i: %ls", this, Track.StreamIndex, *WmfMedia::ResultToString(Result));
		return false;
	}

	UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Added stream %i to topology", this, Track.StreamIndex);

	return true;
}


bool FWmfMediaTracks::AddStreamToTracks(uint32 StreamIndex, bool IsVideoDevice, FString& OutInfo)
{
	OutInfo += FString::Printf(TEXT("Stream %i\n"), StreamIndex);

	// get stream descriptor
	TComPtr<IMFStreamDescriptor> StreamDescriptor;
	{
		BOOL Selected = FALSE;
		HRESULT Result = PresentationDescriptor->GetStreamDescriptorByIndex(StreamIndex, &Selected, &StreamDescriptor);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get stream descriptor for stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
			OutInfo += TEXT("\tmissing stream descriptor\n");

			return false;
		}

		if (Selected == TRUE)
		{
			Result = PresentationDescriptor->DeselectStream(StreamIndex);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to deselect stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
			}
		}
	}

	// get media type handler
	TComPtr<IMFMediaTypeHandler> Handler;
	{
		const HRESULT Result = StreamDescriptor->GetMediaTypeHandler(&Handler);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get media type handler for stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
			OutInfo += TEXT("\tno handler available\n");

			return false;
		}
	}

	// skip unsupported handler types
	GUID MajorType;
	{
		const HRESULT Result = Handler->GetMajorType(&MajorType);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to determine major type of stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
			OutInfo += TEXT("\tfailed to determine MajorType\n");

			return false;
		}

		UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Major type of stream %i is %ls", this, StreamIndex, *WmfMedia::MajorTypeToString(MajorType));
		OutInfo += FString::Printf(TEXT("\tType: %s\n"), *WmfMedia::MajorTypeToString(MajorType));

		if ((MajorType != MFMediaType_Audio) &&
			(MajorType != MFMediaType_SAMI) &&
			(MajorType != MFMediaType_Video))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Unsupported major type %ls of stream %i", this, *WmfMedia::MajorTypeToString(MajorType), StreamIndex);
			OutInfo += TEXT("\tUnsupported stream type\n");

			return false;
		}
	}

	// @todo gmp: handle protected content
	const bool Protected = ::MFGetAttributeUINT32(StreamDescriptor, MF_SD_PROTECTED, FALSE) != 0;
	{
		if (Protected)
		{
			OutInfo += FString::Printf(TEXT("\tProtected content\n"));
		}
	}

	// get number of track formats
	DWORD NumMediaTypes = 0;
	{
		const HRESULT Result = Handler->GetMediaTypeCount(&NumMediaTypes);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get number of track formats in stream %i", this, StreamIndex);
			OutInfo += TEXT("\tfailed to get track formats\n");

			return false;
		}
	}

	// create & add track
	FTrack* Track = nullptr;
	int32 TrackIndex = INDEX_NONE;
	int32* SelectedTrack = nullptr;

	if (MajorType == MFMediaType_Audio)
	{
		SelectedTrack = &SelectedAudioTrack;
		TrackIndex = AudioTracks.AddDefaulted();
		Track = &AudioTracks[TrackIndex];
	}
	else if (MajorType == MFMediaType_SAMI)
	{
		SelectedTrack = &SelectedCaptionTrack;
		TrackIndex = CaptionTracks.AddDefaulted();
		Track = &CaptionTracks[TrackIndex];
	}
	else if (MajorType == MFMediaType_Binary)
	{
		SelectedTrack = &SelectedMetadataTrack;
		TrackIndex = MetadataTracks.AddDefaulted();
		Track = &MetadataTracks[TrackIndex];
	}
	else if (MajorType == MFMediaType_Video)
	{
		SelectedTrack = &SelectedVideoTrack;
		TrackIndex = VideoTracks.AddDefaulted();
		Track = &VideoTracks[TrackIndex];
	}

	check(Track != nullptr);
	check(TrackIndex != INDEX_NONE);
	check(SelectedTrack != nullptr);

	// get current format
	TComPtr<IMFMediaType> CurrentMediaType;
	{
		HRESULT Result = Handler->GetCurrentMediaType(&CurrentMediaType);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get current media type in stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
		}
	}

	Track->SelectedFormat = INDEX_NONE;

	// add track formats
	const UWmfMediaSettings* WmfMediaSettings = GetDefault<UWmfMediaSettings>();
	const bool AllowNonStandardCodecs = WmfMediaSettings->AllowNonStandardCodecs ||
		WmfMediaSettings->bAreHardwareAcceleratedCodecRegistered;

	for (DWORD TypeIndex = 0; TypeIndex < NumMediaTypes; ++TypeIndex)
	{
		OutInfo += FString::Printf(TEXT("\tFormat %u\n"), (uint32)TypeIndex);

		// get media type
		TComPtr<IMFMediaType> MediaType;

		if (FAILED(Handler->GetMediaTypeByIndex(TypeIndex, &MediaType)))
		{
			OutInfo += TEXT("\t\tfailed to get media type\n");

			continue;
		}

		// get sub-type
		GUID SubType;

		if (MajorType == MFMediaType_SAMI)
		{
			FMemory::Memzero(SubType);
		}
		else
		{
			const HRESULT Result = MediaType->GetGUID(MF_MT_SUBTYPE, &SubType);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get sub-type of format %i in stream %i: %ls", this, TypeIndex, StreamIndex, *WmfMedia::ResultToString(Result));
				OutInfo += TEXT("\t\tfailed to get sub-type\n");

				continue;
			}
		}

		const FString TypeName = WmfMedia::SubTypeToString(SubType);
		OutInfo += FString::Printf(TEXT("\t\tCodec: %s\n"), *TypeName);

		// create output type
		TComPtr<IMFMediaType> OutputType = WmfMedia::CreateOutputType(*MediaType, AllowNonStandardCodecs, IsVideoDevice);

		if (!OutputType.IsValid())
		{
			OutInfo += TEXT("\t\tfailed to create output type\n");

			continue;
		}

		// add format details
		int32 FormatIndex = INDEX_NONE;

		if (MajorType == MFMediaType_Audio)
		{
			const uint32 BitsPerSample = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_BITS_PER_SAMPLE, 16);
			const uint32 NumChannels = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_NUM_CHANNELS, 0);
			const uint32 SampleRate = ::MFGetAttributeUINT32(MediaType, MF_MT_AUDIO_SAMPLES_PER_SECOND, 0);

			FormatIndex = Track->Formats.Add({
				MediaType,
				OutputType,
				TypeName,
				{
					BitsPerSample,
					NumChannels,
					SampleRate
				},
				{ 0 }
			});

			OutInfo += FString::Printf(TEXT("\t\tChannels: %i\n"), NumChannels);
			OutInfo += FString::Printf(TEXT("\t\tSample Rate: %i Hz\n"), SampleRate);
			OutInfo += FString::Printf(TEXT("\t\tBits Per Sample: %i\n"), BitsPerSample);
		}
		else if (MajorType == MFMediaType_SAMI)
		{
			FormatIndex = Track->Formats.Add({
				MediaType,
				OutputType,
				TypeName,
				{ 0 },
				{ 0 }
			});
		}
		else if (MajorType == MFMediaType_Binary)
		{
			FormatIndex = Track->Formats.Add({
				MediaType,
				OutputType,
				TypeName,
				{ 0 },
				{ 0 }
			});
		}
		else if (MajorType == MFMediaType_Video)
		{
			GUID OutputSubType;
			{
				const HRESULT Result = OutputType->GetGUID(MF_MT_SUBTYPE, &OutputSubType);

				if (FAILED(Result))
				{
					UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Failed to get video output sub-type for stream %i: %ls", this, StreamIndex, *WmfMedia::ResultToString(Result));
					OutInfo += FString::Printf(TEXT("\t\tfailed to get sub-type"));

					continue;
				}
			}

			const uint32 BitRate = ::MFGetAttributeUINT32(MediaType, MF_MT_AVG_BITRATE, 0);

			float FrameRate;
			{
				UINT32 Numerator = 0;
				UINT32 Denominator = 1;

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE, &Numerator, &Denominator)))
				{
					FrameRate = static_cast<float>(Numerator) / Denominator;
					OutInfo += FString::Printf(TEXT("\t\tFrame Rate: %g fps\n"), FrameRate);
				}
				else
				{
					FrameRate = 0.0f;
					OutInfo += FString::Printf(TEXT("\t\tFrame Rate: n/a\n"));
				}
			}

			TRange<float> FrameRates;
			{
				UINT32 Numerator = 0;
				UINT32 Denominator = 1;
				float Min = -1.0f;
				float Max = -1.0f;

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE_RANGE_MIN, &Numerator, &Denominator)))
				{
					Min = static_cast<float>(Numerator) / Denominator;
				}

				if (SUCCEEDED(::MFGetAttributeRatio(MediaType, MF_MT_FRAME_RATE_RANGE_MAX, &Numerator, &Denominator)))
				{
					Max = static_cast<float>(Numerator) / Denominator;
				}

				if ((Min >= 0.0f) && (Max >= 0.0f))
				{
					FrameRates = TRange<float>::Inclusive(Min, Max);
				}
				else
				{
					FrameRates = TRange<float>(FrameRate);
				}

				OutInfo += FString::Printf(TEXT("\t\tFrame Rate Range: %g - %g fps\n"), FrameRates.GetLowerBoundValue(), FrameRates.GetUpperBoundValue());

				if (FrameRates.IsDegenerate() && (FrameRates.GetLowerBoundValue() == 1.0f))
				{
					OutInfo += FString::Printf(TEXT("\t\tpossibly a still image stream (may not work)\n"));
				}
			}

			// Note: Windows Media Foundation incorrectly exposes still image streams as video streams.
			// Still image streams require special handling and are currently not supported. There is no
			// good way to distinguish these from actual video streams other than that their only supported
			// frame rate is 1 fps, so we skip all 1 fps video streams here.

			if (IsVideoDevice && FrameRates.IsDegenerate() && (FrameRates.GetLowerBoundValue() == 1.0f))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Skipping stream %i, because it is most likely a still image stream", this, StreamIndex);
				OutInfo += FString::Printf(TEXT("\t\tlikely an unsupported still image stream\n"));

				continue;
			}

			FIntPoint OutputDim;
			{
				if (SUCCEEDED(::MFGetAttributeSize(MediaType, MF_MT_FRAME_SIZE, (UINT32*)&OutputDim.X, (UINT32*)&OutputDim.Y)))
				{
					OutInfo += FString::Printf(TEXT("\t\tDimensions: %i x %i\n"), OutputDim.X, OutputDim.Y);
				}
				else
				{
					OutputDim = FIntPoint::ZeroValue;
					OutInfo += FString::Printf(TEXT("\t\tDimensions: n/a\n"));
				}
			}

			FIntPoint BufferDim;
			uint32 BufferStride;
			EMediaTextureSampleFormat SampleFormat;
			{
				if (OutputSubType == MFVideoFormat_NV12)
				{
					if (IsVideoDevice)
					{
						BufferDim.X = OutputDim.X;
						BufferDim.Y = OutputDim.Y * 3 / 2;
					}
					else
					{
						BufferDim.X = Align(OutputDim.X, 16);

						if ((SubType == MFVideoFormat_H264) || (SubType == MFVideoFormat_H264_ES))
						{
							BufferDim.Y = Align(OutputDim.Y, 16) * 3 / 2;
						}
						else
						{
							BufferDim.Y = OutputDim.Y * 3 / 2;
						}
					}

					BufferStride = BufferDim.X;
					SampleFormat = EMediaTextureSampleFormat::CharNV12;
				}
				else if (OutputSubType == MFVideoFormat_RGB32)
				{
					BufferDim = OutputDim;
					BufferStride = OutputDim.X * 4;
					SampleFormat = EMediaTextureSampleFormat::CharBMP;
				}
				else
				{
					int32 AlignedOutputX = OutputDim.X;

					if ((SubType == MFVideoFormat_H264) || (SubType == MFVideoFormat_H264_ES))
					{
						AlignedOutputX = Align(AlignedOutputX, 16);
					}

					int32 SampleStride = AlignedOutputX * 2; // 2 bytes per pixel

					if (SampleStride < 0)
					{
						SampleStride = -SampleStride;
					}

					BufferDim = FIntPoint(AlignedOutputX / 2, OutputDim.Y); // 2 pixels per texel
					BufferStride = SampleStride;
					SampleFormat = EMediaTextureSampleFormat::CharYUY2;
				}
			}

			GUID FormatType = GUID_NULL;

			// prevent duplicates for legacy DirectShow media types
			// see: https://msdn.microsoft.com/en-us/library/windows/desktop/ff485858(v=vs.85).aspx

			if (SUCCEEDED(MediaType->GetGUID(MF_MT_AM_FORMAT_TYPE, &FormatType)))
			{
				if (FormatType == FORMAT_VideoInfo)
				{
					for (int32 Index = Track->Formats.Num() - 1; Index >= 0; --Index)
					{
						const FFormat& Format = Track->Formats[Index];

						if ((Format.Video.FormatType == FORMAT_VideoInfo2) &&
							(Format.Video.FrameRates == FrameRates) &&
							(Format.Video.OutputDim == OutputDim) &&
							(Format.TypeName == TypeName))
						{
							FormatIndex = Index; // keep newer format

							break;
						}
					}
				}
				else if (FormatType == FORMAT_VideoInfo2)
				{
					for (int32 Index = Track->Formats.Num() - 1; Index >= 0; --Index)
					{
						FFormat& Format = Track->Formats[Index];

						if ((Format.Video.FormatType == FORMAT_VideoInfo) &&
							(Format.Video.FrameRates == FrameRates) &&
							(Format.Video.OutputDim == OutputDim) &&
							(Format.TypeName == TypeName))
						{
							Format.InputType = MediaType; // replace legacy format
							FormatIndex = Index;

							break;
						}
					}
				}
			}

			if (FormatIndex == INDEX_NONE)
			{
				FormatIndex = Track->Formats.Add({
					MediaType,
					OutputType,
					TypeName,
					{ 0 },
					{
						BitRate,
						BufferDim,
						BufferStride,
						FormatType,
						FrameRate,
						FrameRates,
						OutputDim,
						SampleFormat
					}
				});
			}
		}
		else
		{
			check(false); // should never get here
		}

		if (MediaType == CurrentMediaType)
		{
			Track->SelectedFormat = FormatIndex;
		}
	}

	// ensure that a track format is selected
	if (Track->SelectedFormat == INDEX_NONE)
	{
		for (int32 FormatIndex = 0; FormatIndex < Track->Formats.Num(); ++FormatIndex)
		{
			const FFormat& Format = Track->Formats[FormatIndex];
			const HRESULT Result = Handler->SetCurrentMediaType(Format.InputType);

			if (SUCCEEDED(Result))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: Picked default format %i for stream %i", this, FormatIndex, StreamIndex);
				Track->SelectedFormat = FormatIndex;
				break;
			}
		}

		if (Track->SelectedFormat == INDEX_NONE)
		{
			UE_LOGF(LogWmfMedia, Verbose, "Tracks %p: No supported media types found in stream %i", this, StreamIndex);
			OutInfo += TEXT("\tunsupported media type\n");
		}
	}

	// select this track if no track is selected yet
	if ((*SelectedTrack == INDEX_NONE) && (Track->SelectedFormat != INDEX_NONE))
	{
		const HRESULT Result = PresentationDescriptor->SelectStream(StreamIndex);

		if (SUCCEEDED(Result))
		{
			*SelectedTrack = TrackIndex;
		}
	}

	// set track details
	PWSTR OutString = NULL;
	UINT32 OutLength = 0;

	if (SUCCEEDED(StreamDescriptor->GetAllocatedString(MF_SD_LANGUAGE, &OutString, &OutLength)))
	{
		Track->Language = OutString;
		::CoTaskMemFree(OutString);
	}

#pragma warning(push)
#pragma warning(disable: 6388) // CA Warning: OutLength may not be NULL - According to MS documentation the initial value does not matter & we are sure to query a string type
	if (SUCCEEDED(StreamDescriptor->GetAllocatedString(MF_SD_STREAM_NAME, &OutString, &OutLength)))
	{
		Track->Name = OutString;
		::CoTaskMemFree(OutString);
	}
#pragma warning(pop)

	Track->DisplayName = (Track->Name.IsEmpty())
		? FText::Format(LOCTEXT("UnnamedStreamFormat", "Unnamed Track (Stream {0})"), FText::AsNumber((uint32)StreamIndex))
		: FText::FromString(Track->Name);

	Track->Descriptor = StreamDescriptor;
	Track->Handler = Handler;
	Track->Protected = Protected;
	Track->StreamIndex = StreamIndex;

	return true;
}


const FWmfMediaTracks::FFormat* FWmfMediaTracks::GetAudioFormat(int32 TrackIndex, int32 FormatIndex) const
{
	if (AudioTracks.IsValidIndex(TrackIndex))
	{
		const FTrack& Track = AudioTracks[TrackIndex];

		if (Track.Formats.IsValidIndex(FormatIndex))
		{
			return &Track.Formats[FormatIndex];
		}
	}

	return nullptr;
}


const FWmfMediaTracks::FTrack* FWmfMediaTracks::GetTrack(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return &AudioTracks[TrackIndex];
		}

	case EMediaTrackType::Metadata:
		if (MetadataTracks.IsValidIndex(TrackIndex))
		{
			return &MetadataTracks[TrackIndex];
		}

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return &CaptionTracks[TrackIndex];
		}

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return &VideoTracks[TrackIndex];
		}

	default:
		break; // unsupported track type
	}

	return nullptr;
}


const FWmfMediaTracks::FFormat* FWmfMediaTracks::GetVideoFormat(int32 TrackIndex, int32 FormatIndex) const
{
	if (VideoTracks.IsValidIndex(TrackIndex))
	{
		const FTrack& Track = VideoTracks[TrackIndex];

		if (Track.Formats.IsValidIndex(FormatIndex))
		{
			return &Track.Formats[FormatIndex];
		}
	}

	return nullptr;
}


/* FWmfMediaTracks callbacks
 *****************************************************************************/

void FWmfMediaTracks::HandleMediaSamplerClock(EWmfMediaSamplerClockEvent Event, EMediaTrackType TrackType)
{
	// IMFSampleGrabberSinkCallback callbacks seem to be broken (always returns Stopped)
	// We handle sink synchronization via SetPaused() as a workaround
}


FMediaTimeStamp FWmfMediaTracks::AdjustTimeStamp(FTimespan Time, EMediaTrackType TrackType)
{
	FScopeLock Lock(&CriticalSection);
	return FMediaTimeStamp(Time, SeekIndex, LoopIndex);
}

bool FWmfMediaTracks::IsSampleStillDesired()
{
	FScopeLock Lock(&CriticalSection);
	return SeekIndex >= NextMinSequenceIndex.Get(0);
}


void FWmfMediaTracks::HandleMediaSamplerAudioSample(const uint8* Buffer, uint32 Size, FTimespan /*Duration*/, FTimespan Time)
{
	if (Buffer == nullptr)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);

	if (!AudioTracks.IsValidIndex(SelectedAudioTrack))
	{
		return; // invalid track index
	}

	const FTrack& Track = AudioTracks[SelectedAudioTrack];
	const FFormat* Format = GetAudioFormat(SelectedAudioTrack, Track.SelectedFormat);

	if (Format == nullptr)
	{
		return; // no format selected
	}

	if (AudioSampleQueue.Num() >= FMediaPlayerQueueDepths::MaxAudioSinkDepth)
	{
		return;
	}

	// duration provided by WMF is sometimes incorrect when seeking
	FTimespan Duration = (Size * ETimespan::TicksPerSecond) / (Format->Audio.NumChannels * Format->Audio.SampleRate * sizeof(int16));

	// create & add sample to queue
	const TSharedRef<FWmfMediaAudioSample, ESPMode::ThreadSafe> AudioSample = AudioSamplePool->AcquireShared();

	if (AudioSample->Initialize(Buffer, Size, Format->Audio.NumChannels, Format->Audio.SampleRate, AdjustTimeStamp(Time, EMediaTrackType::Audio), Duration))
	{
		if (AudioSample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
		{
			AudioSampleQueue.Enqueue(AudioSample);
		}
	}
}


void FWmfMediaTracks::HandleMediaSamplerCaptionSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time)
{
	if (Buffer == nullptr)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);

	if (!CaptionTracks.IsValidIndex(SelectedCaptionTrack))
	{
		return; // invalid track index
	}

	if (CaptionSampleQueue.Num() >= FMediaPlayerQueueDepths::MaxCaptionSinkDepth)
	{
		return;
	}

	// create & add sample to queue
	const FTrack& Track = CaptionTracks[SelectedCaptionTrack];
	const auto CaptionSample = MakeShared<FWmfMediaOverlaySample, ESPMode::ThreadSafe>();

	if (CaptionSample->Initialize((char*)Buffer, AdjustTimeStamp(Time, EMediaTrackType::Caption), Duration))
	{
		if (CaptionSample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
		{
			CaptionSampleQueue.Enqueue(CaptionSample);
		}
	}
}


void FWmfMediaTracks::HandleMediaSamplerMetadataSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time)
{
	if (Buffer == nullptr)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);

	if (!MetadataTracks.IsValidIndex(SelectedMetadataTrack))
	{
		return; // invalid track index
	}

	if (MetadataSampleQueue.Num() >= FMediaPlayerQueueDepths::MaxMetadataSinkDepth)
	{
		return;
	}

	// create & add sample to queue
	const FTrack& Track = MetadataTracks[SelectedMetadataTrack];
	const auto BinarySample = MakeShared<FWmfMediaBinarySample, ESPMode::ThreadSafe>();

	if (BinarySample->Initialize(Buffer, Size, AdjustTimeStamp(Time, EMediaTrackType::Metadata), Duration))
	{
		if (BinarySample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
		{
			MetadataSampleQueue.Enqueue(BinarySample);
		}
	}
}


void FWmfMediaTracks::HandleMediaSamplerVideoSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time)
{
	if (Buffer == nullptr)
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);

	if (!VideoTracks.IsValidIndex(SelectedVideoTrack))
	{
		return; // invalid track index
	}

	const FTrack& Track = VideoTracks[SelectedVideoTrack];
	const FFormat* Format = GetVideoFormat(SelectedVideoTrack, Track.SelectedFormat);

	if (Format == nullptr)
	{
		return; // no format selected
	}

	if ((Format->Video.BufferStride * Format->Video.BufferDim.Y) > Size)
	{
		return; // invalid buffer size (can happen during format switch)
	}

	if (VideoSampleQueue.Num() >= FMediaPlayerQueueDepths::MaxVideoSinkDepth)
	{
		return;
	}

	// WMF doesn't report durations for some formats
	if (Duration.IsZero())
	{
		float FrameRate = Format->Video.FrameRate;

		if (FrameRate <= 0.0f)
		{
			FrameRate = 30.0f;
		}

		Duration = FTimespan((int64)((float)ETimespan::TicksPerSecond / FrameRate));
	}

	// create & add sample to queue
	const TSharedRef<FWmfMediaTextureSample, ESPMode::ThreadSafe> TextureSample = VideoSamplePool->AcquireShared();

	if (TextureSample->Initialize(
		Buffer,
		Size,
		Format->Video.BufferDim,
		Format->Video.OutputDim,
		Format->Video.SampleFormat,
		Format->Video.BufferStride,
		AdjustTimeStamp(Time, EMediaTrackType::Video),
		Duration))
	{
		if (TextureSample->GetTime().GetSequenceIndex() >= NextMinSequenceIndex.Get(0))
		{
			VideoSampleQueue.Enqueue(TextureSample);
		}
	}
}


#include "Windows/HideWindowsPlatformTypes.h"

#undef LOCTEXT_NAMESPACE

#endif //WMFMEDIA_SUPPORTED_PLATFORM
