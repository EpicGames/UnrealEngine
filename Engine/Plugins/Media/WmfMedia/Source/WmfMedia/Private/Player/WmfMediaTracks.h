// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "IMediaControls.h"
#include "IMediaSamples.h"
#include "IMediaTracks.h"
#include "Math/IntPoint.h"
#include "MediaSampleQueue.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"

#include "Windows/AllowWindowsPlatformTypes.h"

class FWmfMediaAudioSamplePool;
class FWmfMediaSampler;
class FWmfMediaTextureSamplePool;
class FWmfMediaHardwareVideoDecodingTextureSamplePool;
class FWmfMediaSession;
class IMediaAudioSample;
class IMediaBinarySample;
class IMediaOverlaySample;
class IMediaTextureSample;
class FWmfMediaTextureSample;
class FMediaTimeStamp;
class FWmfMediaStreamSink;

enum class EMediaTextureSampleFormat;
enum class EMediaTrackType;
enum class EWmfMediaSamplerClockEvent;

struct FMediaPlayerOptions;

/**
 * Track collection for Windows Media Foundation based media players.
 */
class FWmfMediaTracks
	: public IMediaSamples
	, public IMediaTracks
{
	/** Track format. */
	struct FFormat
	{
		TComPtr<IMFMediaType> InputType;
		TComPtr<IMFMediaType> OutputType;
		FString TypeName;

		struct
		{
			uint32 BitsPerSample;
			uint32 NumChannels;
			uint32 SampleRate;
		}
		Audio;

		struct
		{
			uint32 BitRate;
			FIntPoint BufferDim;
			uint32 BufferStride;
			GUID FormatType;
			float FrameRate;
			TRange<float> FrameRates;
			FIntPoint OutputDim;
			EMediaTextureSampleFormat SampleFormat;
		}
		Video;
	};


	/** Track information. */
	struct FTrack
	{
		TComPtr<IMFStreamDescriptor> Descriptor;
		FText DisplayName;
		TArray<FFormat> Formats;
		TComPtr<IMFMediaTypeHandler> Handler;
		FString Language;
		FString Name;
		bool Protected;
		int32 SelectedFormat;
		DWORD StreamIndex;
	};

public:

	/** Default constructor. */
	FWmfMediaTracks();

	/** Virtual destructor. */
	virtual ~FWmfMediaTracks();

public:

	/**
	 * Append track statistics information to the given string.
	 *
	 * @param OutStats The string to append the statistics to.
	 */
	void AppendStats(FString& OutStats) const;

	/**
	 * Clear the streams flags.
	 *
	 * @see GetFlags
	 */
	void ClearFlags();

	/**
	 * Create the playback topology for the current track selection.
	 *
	 * @return The new topology.
	 */
	TComPtr<IMFTopology> CreateTopology();

	/**
	 * Get the total duration of the current media source.
	 *
	 * @return Media duration.
	 * @see GetInfo, GetSamples, Initialize
	 */
	FTimespan GetDuration() const;

	/**
	 * Get the current flags.
	 *
	 * @param OutMediaSourceChanged Will indicate whether the media source changed.
	 * @param OutSelectionChanged Will indicate whether the track selection changed.
	 * @see ClearFlags
	 */
	void GetFlags(bool& OutMediaSourceChanged, bool& OutSelectionChanged) const;

	/**
	 * Get the information string for the currently loaded media source.
	 *
	 * @return Info string.
	 * @see GetDuration, GetSamples
	 */
	const FString& GetInfo() const
	{
		return Info;
	}

	/**
	 * Initialize the track collection.
	 *
	 * @param InMediaSource The media source object.
	 * @param Url The media source URL.
	 * @see IsInitialized, Shutdown
	 */
	void Initialize(IMFMediaSource* InMediaSource, const FString& Url, const FMediaPlayerOptions* PlayerOptions);

	/**
	 * Reinitialize the track collection
	 *
	 * @see IsInitialized, Shutdown
	 */
	void ReInitialize();

	/**
	 * Whether this object has been initialized.
	 *
	 * @return true if initialized, false otherwise.
	 * @see Initialize, Shutdown
	 */
	bool IsInitialized() const
	{
		return (MediaSource != NULL);
	}

	/**
	 * Shut down the track collection.
	 *
	 * @see Initialize, IsInitialized
	 */
	void Shutdown();

	/**
	 * Call this to pass on the session state to us.
	 *
	 * @param InState State of session.
	 */
	void SetSessionState(EMediaState InState);

public:

	/**
	 * Notify track of triggered seek
	 */
	void SeekStarted(const FTimespan& InTime, int32 InNewSequenceIndex);

	/**
	 * Notify track of triggered loop
	 */
	void LoopStarted(float UnpausedSessionRate);

	void SetSession(FWmfMediaSession* InSession)
	{
		Session = InSession;
	}

	void SessionEnded();

	void RequestMoreVideoDataFromStreamSink();

	bool ExecuteOnceMediaStreamSinkHasNoPendingRequests(TFunction<void()>&& ExecuteOnIdle);

public:

	//~ IMediaSamples interface

	virtual bool FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override { return FetchAudio(TimeRange.GetLowerBound().IsOpen() ? TRange<FMediaTimeStamp>() : TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue(), 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue(), 0)), OutSample); }
	virtual bool FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override { return FetchCaption(TimeRange.GetLowerBound().IsOpen() ? TRange<FMediaTimeStamp>() : TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue(), 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue(), 0)), OutSample); }
	virtual bool FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override { return FetchMetadata(TimeRange.GetLowerBound().IsOpen() ? TRange<FMediaTimeStamp>() : TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue(), 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue(), 0)), OutSample); }
	virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override { return FetchVideo(TimeRange.GetLowerBound().IsOpen() ? TRange<FMediaTimeStamp>() : TRange<FMediaTimeStamp>(FMediaTimeStamp(TimeRange.GetLowerBoundValue(), 0), FMediaTimeStamp(TimeRange.GetUpperBoundValue(), 0)), OutSample); }

	virtual bool FetchAudio(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override;
	virtual bool FetchCaption(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	virtual bool FetchMetadata(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override;
	virtual bool FetchVideo(TRange<FMediaTimeStamp> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override;

	virtual void FlushSamples() override;

	virtual EFetchBestSampleResult FetchBestVideoSampleForTimeRange(const TRange<FMediaTimeStamp>& TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample, bool bReverse, bool bConsistentResult) override;
	virtual void SetMinExpectedNextSequenceIndex(TOptional<int32> InNextSequenceIndex) override;

	virtual bool DiscardVideoSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override;
	virtual bool DiscardAudioSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override;
	virtual bool DiscardCaptionSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override;
	virtual bool DiscardMetadataSamples(const TRange<FMediaTimeStamp>& TimeRange, bool bReverse) override;
	virtual bool PeekVideoSampleTime(FMediaTimeStamp & TimeStamp) override;

public:

	//~ IMediaTracks interface

	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;
	virtual bool SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate) override;

protected:

	/**
	 * Add the specified stream to the track collection.
	 *
	 * @param StreamIndex The index of the stream to add.
	 * @param OutInfo Will contain appended debug information.
	 * @param IsVideoDevice Whether the stream belongs to a video capture device.
	 * @return true on success, false otherwise.
	 * @see AddTrackToTopology
	 */
	bool AddStreamToTracks(uint32 StreamIndex, bool IsVideoDevice, FString& OutInfo);

	/**
	 * Add the given track to the specified playback topology.
	 *
	 * @param Track The track to add.
	 * @param Topology The playback topology.
	 * @return true on success, false otherwise.
	 * @see AddStreamToTracks
	 */
	bool AddTrackToTopology(const FTrack& Track, IMFTopology& Topology);

private:

	/**
	 * Get the specified audio format.
	 *
	 * @param TrackIndex Index of the audio track that contains the format.
	 * @param FormatIndex Index of the format to return.
	 * @return Pointer to format, or nullptr if not found.
	 * @see GetVideoFormat
	 */
	const FFormat* GetAudioFormat(int32 TrackIndex, int32 FormatIndex) const;

	/**
	 * Get the specified track information.
	 *
	 * @param TrackType The type of track.
	 * @param TrackIndex Index of the track to return.
	 * @return Pointer to track, or nullptr if not found.
	 */
	const FTrack* GetTrack(EMediaTrackType TrackType, int32 TrackIndex) const;

	/**
	 * Get the specified video format.
	 *
	 * @param TrackIndex Index of the video track that contains the format.
	 * @param FormatIndex Index of the format to return.
	 * @return Pointer to format, or nullptr if not found.
	 * @see GetAudioFormat
	 */
	const FFormat* GetVideoFormat(int32 TrackIndex, int32 FormatIndex) const;

private:

	/** Callback for handling media sampler pauses. */
	void HandleMediaSamplerClock(EWmfMediaSamplerClockEvent Event, EMediaTrackType TrackType);

	/** Callback for handling new samples from the streams' media sample buffers. */
	void HandleMediaSamplerAudioSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time);

	/** Callback for handling new caption samples. */
	void HandleMediaSamplerCaptionSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time);

	/** Callback for handling new metadata samples. */
	void HandleMediaSamplerMetadataSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time);

	/** Callback for handling new video samples. */
	void HandleMediaSamplerVideoSample(const uint8* Buffer, uint32 Size, FTimespan Duration, FTimespan Time);

	/** Make sure sequence index is up to date and combine with given timestamp */
	FMediaTimeStamp AdjustTimeStamp(FTimespan Time, EMediaTrackType TrackType);
	
	/** Check if the current sequence index to apply is still the one that is wanted. */
	bool IsSampleStillDesired();

private:
	/** Audio sample object pool. */
	FWmfMediaAudioSamplePool* AudioSamplePool;

	/** Audio sample queue. */
	TMediaSampleQueue<IMediaAudioSample> AudioSampleQueue;

	/** The available audio tracks. */
	TArray<FTrack> AudioTracks;

	/** Overlay sample queue. */
	TMediaSampleQueue<IMediaOverlaySample> CaptionSampleQueue;

	/** The available caption tracks. */
	TArray<FTrack> CaptionTracks;

	/** Synchronizes write access to track arrays, selections & sinks. */
	mutable FCriticalSection CriticalSection;

	/** Media information string. */
	FString Info;

	/** The initial media url. */
	FString SourceUrl;

	/** The currently opened media. */
	TComPtr<IMFMediaSource> MediaSource;

	// Hardware Acccelerated Stream Sink
	TComPtr<FWmfMediaStreamSink> MediaStreamSink;

	/** Whether the media source has changed. */
	bool MediaSourceChanged;

	/** Metadata sample queue. */
	TMediaSampleQueue<IMediaBinarySample> MetadataSampleQueue;

	/** The available metadata tracks. */
	TArray<FTrack> MetadataTracks;

	/** The presentation descriptor of the currently opened media. */
	TComPtr<IMFPresentationDescriptor> PresentationDescriptor;

	/** Index of the selected audio track. */
	int32 SelectedAudioTrack;

	/** Index of the selected caption track. */
	int32 SelectedCaptionTrack;

	/** Index of the selected binary track. */
	int32 SelectedMetadataTrack;

	/** Index of the selected video track. */
	int32 SelectedVideoTrack;

	/** Whether the track selection changed. */
	bool SelectionChanged;

	/** Flag to display if the topology node has requested hardware acceleration. */
	bool bVideoTrackRequestedHardwareAcceleration;

	/** Video sample object pool. */
	FWmfMediaTextureSamplePool* VideoSamplePool;
	TSharedPtr<FWmfMediaHardwareVideoDecodingTextureSamplePool> VideoHardwareVideoDecodingSamplePool;

	/** Video sample queue. */
	TMediaSampleQueue<IMediaTextureSample> VideoSampleQueue;

	/** The available video tracks. */
	TArray<FTrack> VideoTracks;

	/** Caches the value from GetDuration. */
	FTimespan CachedDuration;

	/** Current state of the session. */
	EMediaState SessionState;

	FWmfMediaSession* Session;

	/** SequenceIndex tracking */
	TOptional<int32> NextMinSequenceIndex;
	int32 SeekIndex;
	int32 LoopIndex;
};


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
