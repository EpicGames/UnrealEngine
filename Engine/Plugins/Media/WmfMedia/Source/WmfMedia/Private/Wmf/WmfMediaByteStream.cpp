// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaByteStream.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "WmfMediaReadState.h"
#include "Misc/ScopeLock.h"

#include "Windows/AllowWindowsPlatformTypes.h"


/* FWmfByteStream structors
 *****************************************************************************/

FWmfMediaByteStream::FWmfMediaByteStream(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive)
	: AsyncReadInProgress(false)
	, Archive(InArchive)
	, RefCount(0)
{ }


FWmfMediaByteStream::~FWmfMediaByteStream()
{
	check(RefCount == 0);
}


/* IMFAsyncCallback interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FWmfMediaByteStream::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


STDMETHODIMP FWmfMediaByteStream::GetParameters(unsigned long*, unsigned long*)
{
	return E_NOTIMPL;
}


STDMETHODIMP FWmfMediaByteStream::Invoke(IMFAsyncResult* AsyncResult)
{
	// recover read state
	IUnknown* State = NULL;
	
	if (FAILED(AsyncResult->GetState(&State)))
	{
		return S_OK;
	}

	IMFAsyncResult* CallerResult = NULL;

	if (FAILED(State->QueryInterface(IID_PPV_ARGS(&CallerResult))))
	{
		return S_OK;
	}

	IUnknown* Unknown = NULL;
	
	if ((CallerResult != NULL) && FAILED(CallerResult->GetObject(&Unknown)))
	{
		return S_OK;
	}

	FWmfMediaReadState* ReadState = static_cast<FWmfMediaReadState*>(Unknown);

	// perform the read
	ULONG cbRead;
	Read(ReadState->GetReadBuffer(), ReadState->GetReadBufferSize() - ReadState->GetBytesRead(), &cbRead);
	ReadState->AddBytesRead(cbRead);

	// notify caller
	if (CallerResult != NULL)
	{
		CallerResult->SetStatus(S_OK);
		::MFInvokeCallback(CallerResult);
	}

	// clean up
	if (CallerResult != NULL)
	{
		CallerResult->Release();
	}

	if (Unknown != NULL)
	{
		Unknown->Release();
	}

	if (State != NULL)
	{
		State->Release();
	}

	return S_OK;
}


#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

STDMETHODIMP FWmfMediaByteStream::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FWmfMediaByteStream, IMFByteStream),
		QITABENT(FWmfMediaByteStream, IMFAsyncCallback),
		{ 0 }
	};

	return QISearch( this, QITab, RefID, Object );
}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif


STDMETHODIMP_(ULONG) FWmfMediaByteStream::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);
	
	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


/* IMFByteStream interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaByteStream::BeginRead(BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState)
{
	if ((pCallback == NULL) || (pb == NULL))
	{
		return E_INVALIDARG;
	}

	TComPtr<FWmfMediaReadState> ReadState = new FWmfMediaReadState(pb, cb);

	if (ReadState == NULL)
	{
		return E_OUTOFMEMORY;
	}

	IMFAsyncResult* AsyncResult = NULL;
	HRESULT Result = ::MFCreateAsyncResult(ReadState, pCallback, punkState, &AsyncResult);

	if (SUCCEEDED(Result))
	{
		AsyncReadInProgress = true;
		Result = ::MFPutWorkItem(MFASYNC_CALLBACK_QUEUE_STANDARD, this, AsyncResult);
		AsyncResult->Release();
	}

	return Result;
}


STDMETHODIMP FWmfMediaByteStream::BeginWrite(const BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState)
{
	return E_NOTIMPL;
}


STDMETHODIMP FWmfMediaByteStream::Close()
{
	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::EndRead(IMFAsyncResult* pResult, ULONG* pcbRead)
{
	if (pcbRead == NULL)
	{
		return E_INVALIDARG;
	}

	IUnknown* Unknown;
	{
		pResult->GetObject(&Unknown);
		FWmfMediaReadState* ReadState = static_cast<FWmfMediaReadState*>(Unknown);
		*pcbRead = (ULONG)ReadState->GetBytesRead();

		Unknown->Release();
	}

	AsyncReadInProgress = false;

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::EndWrite(IMFAsyncResult* pResult, ULONG* pcbWritten)
{
	return E_NOTIMPL;
}


STDMETHODIMP FWmfMediaByteStream::Flush()
{
	return E_NOTIMPL;
}


STDMETHODIMP FWmfMediaByteStream::GetCapabilities(DWORD* pdwCapabilities)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pdwCapabilities = MFBYTESTREAM_IS_READABLE | MFBYTESTREAM_IS_SEEKABLE;

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::GetCurrentPosition(QWORD* pqwPosition)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pqwPosition = Archive->Tell();

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::GetLength(QWORD* pqwLength)
{
	FScopeLock ScopeLock(&CriticalSection);
	*pqwLength = (QWORD)Archive->TotalSize();

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::IsEndOfStream(BOOL* pfEndOfStream)
{
	if (pfEndOfStream == NULL)
	{
		return E_INVALIDARG;
	}

	FScopeLock ScopeLock(&CriticalSection);
	*pfEndOfStream = Archive->AtEnd() ? TRUE : FALSE;

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::Read(BYTE* pb, ULONG cb, ULONG* pcbRead)
{
	FScopeLock ScopeLock(&CriticalSection);

	int64 Position = Archive->Tell();
	int64 Size = Archive->TotalSize();
	ULONG BytesToRead = cb;

	if (BytesToRead > (ULONG)Size)
	{
		BytesToRead = Size;
	}

	if ((Size - BytesToRead) < Archive->Tell())
	{
		BytesToRead = Size - Position;
	}

	if (BytesToRead > 0)
	{
		Archive->Serialize(pb, BytesToRead);
	}

	if (pcbRead != NULL)
	{
		*pcbRead = BytesToRead;
	}	

	Archive->Seek(Position + BytesToRead);

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::Seek(MFBYTESTREAM_SEEK_ORIGIN SeekOrigin, LONGLONG qwSeekOffset, DWORD dwSeekFlags, QWORD* pqwCurrentPosition)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (AsyncReadInProgress)
	{
		return S_FALSE;
	}

	if (SeekOrigin == msoCurrent)
	{
		Archive->Seek(Archive->Tell() + qwSeekOffset);
	}
	else
	{
		Archive->Seek(qwSeekOffset);
	}

	if (pqwCurrentPosition != NULL)
	{
		*pqwCurrentPosition = (QWORD)Archive->Tell();
	}

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::SetLength(QWORD qwLength)
{
	return E_NOTIMPL;
}


STDMETHODIMP FWmfMediaByteStream::SetCurrentPosition(QWORD qwPosition)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (AsyncReadInProgress)
	{
		return S_FALSE;
	}

	if (qwPosition > (QWORD)Archive->TotalSize())
	{
		// MSDN is wrong: clamp instead of returning E_INVALIDARG
		qwPosition = (QWORD)Archive->TotalSize();
	}

	Archive->Seek((int64)qwPosition);

	return S_OK;
}


STDMETHODIMP FWmfMediaByteStream::Write(const BYTE* pb, ULONG cb, ULONG* pcbWritten)
{
	return E_NOTIMPL;
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
