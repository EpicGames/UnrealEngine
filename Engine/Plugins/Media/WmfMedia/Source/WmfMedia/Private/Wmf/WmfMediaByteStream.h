// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Windows/AllowWindowsPlatformTypes.h"


/**
 * Implements a wrapper for media source data that is played from memory.
 */
class FWmfMediaByteStream
	: public IMFAsyncCallback
	, public IMFByteStream
{
public:

	/**
	 * Creates and initializes a new instance from the specified buffer.
	 *
	 * @param InArchive The archive to stream from.
	 */
	FWmfMediaByteStream(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive);

public:

	//~ IMFAsyncCallback interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP GetParameters(unsigned long*, unsigned long*);
	STDMETHODIMP Invoke(IMFAsyncResult* AsyncResult);
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

public:

	//~ IMFByteStream interface

	STDMETHODIMP BeginRead(BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState);
	STDMETHODIMP BeginWrite(const BYTE* pb, ULONG cb, IMFAsyncCallback* pCallback, IUnknown* punkState);
	STDMETHODIMP Close();
	STDMETHODIMP EndRead(IMFAsyncResult* pResult, ULONG* pcbRead);
	STDMETHODIMP EndWrite(IMFAsyncResult* pResult, ULONG* pcbWritten);
	STDMETHODIMP Flush();
	STDMETHODIMP GetCapabilities(DWORD* pdwCapabilities);
	STDMETHODIMP GetCurrentPosition(QWORD* pqwPosition);
	STDMETHODIMP GetLength(QWORD* pqwLength);
	STDMETHODIMP IsEndOfStream(BOOL* pfEndOfStream);
	STDMETHODIMP Read(BYTE* pb, ULONG cb, ULONG* pcbRead);
	STDMETHODIMP Seek(MFBYTESTREAM_SEEK_ORIGIN SeekOrigin, LONGLONG qwSeekOffset, DWORD dwSeekFlags, QWORD* pqwCurrentPosition);
	STDMETHODIMP SetLength(QWORD qwLength);
	STDMETHODIMP SetCurrentPosition(QWORD qwPosition);
	STDMETHODIMP Write(const BYTE* pb, ULONG cb, ULONG* pcbWritten);

private:

	/** Hidden destructor. */
	virtual ~FWmfMediaByteStream();

private:

	/** Whether the stream is currently being read asynchronously. */
	bool AsyncReadInProgress;

	/** Holds the archive to stream from. */
	TSharedRef<FArchive, ESPMode::ThreadSafe> Archive;

	/** Critical section for locking access to this class. */
	mutable FCriticalSection CriticalSection;

	/** Holds a reference counter for this instance. */
	int32 RefCount;
};


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
