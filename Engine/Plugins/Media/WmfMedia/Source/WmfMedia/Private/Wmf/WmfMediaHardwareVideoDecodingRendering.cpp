// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_WINDOWS

#include "WmfMediaHardwareVideoDecodingRendering.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "GlobalShader.h"
#include "Logging/LogMacros.h"
#include "MediaShaders.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "SceneUtils.h"
#include "SceneInterface.h"
#include "ShaderParameterUtils.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#include "ID3D11DynamicRHI.h"
#include "DynamicRHI.h"

#include "WmfMediaHardwareVideoDecodingTextureSample.h"
#include "WmfMediaCommon.h"

#include "dxgi.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d11.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "WmfMediaHardwareVideoDecodingShaders.h"

DECLARE_GPU_STAT_NAMED(MediaTextureConversion, TEXT("MediaTextureConversion"));


struct FRHICommandCopyResource final : public FRHICommand<FRHICommandCopyResource>
{
	TComPtr<ID3D11Texture2D> SampleTexture;
	FTextureRHIRef SampleDestinationTexture;

	FRHICommandCopyResource(ID3D11Texture2D* InSampleTexture, FRHITexture* InSampleDestinationTexture)
		: SampleTexture(InSampleTexture)
		, SampleDestinationTexture(InSampleDestinationTexture)
	{
	}

	void Execute(FRHICommandListBase& CmdList)
	{
		LLM_SCOPE(ELLMTag::MediaStreaming);
		ID3D11Device* D3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();
		ID3D11DeviceContext* D3D11DeviceContext = GetID3D11DynamicRHI()->RHIGetDeviceContext();

		if (D3D11DeviceContext)
		{
			ID3D11Resource* DestinationTexture = GetID3D11DynamicRHI()->RHIGetResource(SampleDestinationTexture);
			if (DestinationTexture)
			{
				TComPtr<IDXGIResource> OtherResource(nullptr);
				SampleTexture->QueryInterface(__uuidof(IDXGIResource), (void**)&OtherResource);

				if (OtherResource)
				{
					HANDLE SharedHandle = nullptr;
					if (OtherResource->GetSharedHandle(&SharedHandle) == S_OK)
					{
						if (SharedHandle != 0)
						{
							TComPtr<ID3D11Resource> SharedResource;
							D3D11Device->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (void**)&SharedResource);

							if (SharedResource)
							{
								TComPtr<IDXGIKeyedMutex> KeyedMutex;
								SharedResource->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);

								if (KeyedMutex)
								{
									// Key is 1 : Texture as just been updated
									// Key is 2 : Texture as already been updated.
									// Do not wait to acquire key 1 since there is race no condition between writer and reader.
									if (KeyedMutex->AcquireSync(1, 0) == S_OK)
									{
										// Copy from shared texture of FWmfMediaSink device to Rendering device
										D3D11DeviceContext->CopyResource(DestinationTexture, SharedResource);
										KeyedMutex->ReleaseSync(2);
									}
									else
									{
										// If key 1 cannot be acquired, another reader is already copying the resource
										// and will release key with 2. 
										// Wait to acquire key 2.
										if (KeyedMutex->AcquireSync(2, INFINITE) == S_OK)
										{
											KeyedMutex->ReleaseSync(2);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
};

bool FWmfMediaHardwareVideoDecodingParameters::ConvertTextureFormat_RenderThread(FRHICommandListImmediate& RHICmdList, FWmfMediaHardwareVideoDecodingTextureSample* InSample, FTextureRHIRef InDstTexture)
{
	LLM_SCOPE(ELLMTag::MediaStreaming);

	if (InSample == nullptr || !InDstTexture.IsValid())
	{
		return false;
	}

	check(InSample);
	
	TComPtr<ID3D11Texture2D> SampleTexture = InSample->GetSourceTexture();

	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaTextureConversion, "FWmfMediaHardwareVideoDecodingParameters_Convert");

		RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::SRVMask, ERHIAccess::RTV));

		FRHIRenderPassInfo RPInfo(InDstTexture, ERenderTargetActions::DontLoad_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertTextureFormat"));

		// Update viewport.
		RHICmdList.SetViewport(0, 0, 0.f, InSample->GetDim().X, InSample->GetDim().Y, 1.f);

		// Set the graphic pipeline state.
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();

		// Copy buffer.
		FTextureRHIRef SampleDestinationTexture = InSample->GetOrCreateDestinationTexture(RHICmdList);
		if (InSample->IsBufferExternal() || !SampleTexture.IsValid())
		{
			const uint8* Data = (const uint8*)InSample->GetBuffer();
			
			uint32 Stride = InSample->GetStride();
			uint32 Height = InSample->GetDim().Y;
			FUpdateTextureRegion2D Region(0, 0, 0, 0, InSample->GetDim().X, Height);
			RHICmdList.UpdateTexture2D(SampleDestinationTexture, 0, Region, Stride, Data);
		}
		else
		{
			if (RHICmdList.Bypass())
			{
				FRHICommandCopyResource Cmd(SampleTexture, SampleDestinationTexture);
				Cmd.Execute(RHICmdList);
			}
			else
			{
				new (RHICmdList.AllocCommand<FRHICommandCopyResource>()) FRHICommandCopyResource(SampleTexture, SampleDestinationTexture);
			}
		}

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		if (InSample->GetFormat() == EMediaTextureSampleFormat::CharNV12)
		{
			TShaderMapRef< FHardwareVideoDecodingVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FHardwareVideoDecodingPS > PixelShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FShaderResourceViewRHIRef Y_SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetMipRange(0, 1)
					.SetFormat(PF_G8));
			FShaderResourceViewRHIRef UV_SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetMipRange(0, 1)
					.SetFormat(PF_R8G8));

			const FHardwareVideoDecodingShaderParams Parameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(Y_SRV, UV_SRV, InSample->IsOutputSrgb());

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);
		}
		else if ((InSample->GetFormat() == EMediaTextureSampleFormat::CharBGRA) ||
			(InSample->GetFormat() == EMediaTextureSampleFormat::DXT1) ||
			(InSample->GetFormat() == EMediaTextureSampleFormat::DXT5))
		{
			TShaderMapRef< FHardwareVideoDecodingVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FHardwareVideoDecodingPassThroughPS > PixelShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			EPixelFormat PixelFormat = PF_B8G8R8A8;
			if (InSample->GetFormat() == EMediaTextureSampleFormat::DXT1)
			{
				PixelFormat = PF_DXT1;
			}
			else if (InSample->GetFormat() == EMediaTextureSampleFormat::DXT5)
			{
				PixelFormat = PF_DXT5;
			}
			FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetFormat(PixelFormat));

			const FHardwareVideoDecodingShaderParams Parameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(SRV, InSample->IsOutputSrgb());

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);
		}
		else if (InSample->GetFormat() == EMediaTextureSampleFormat::Y416)
		{
			TShaderMapRef< FHardwareVideoDecodingVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FHardwareVideoDecodingY416PS > PixelShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetFormat(PF_A16B16G16R16));

			const FHardwareVideoDecodingShaderParams Parameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(SRV, InSample->IsOutputSrgb());

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);
		}
		else if (InSample->GetFormat() == EMediaTextureSampleFormat::YCoCg_DXT5)
		{
			TShaderMapRef< FHardwareVideoDecodingVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FHardwareVideoDecodingYCoCgPS > PixelShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetFormat(PF_DXT5));

			const FHardwareVideoDecodingShaderParams Parameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(SRV, InSample->IsOutputSrgb());

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);
		}
		else if (InSample->GetFormat() == EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4)
		{
			// Set up alpha texture.
			FTextureRHIRef SampleDestinationAlphaTexture = InSample->GetOrCreateDestinationAlphaTexture(RHICmdList);
			{
				const uint8* Data = ((const uint8*)InSample->GetBuffer()) + InSample->GetDim().X * InSample->GetDim().Y;

				uint32 Stride = InSample->GetDim().X * 2;
				uint32 Height = InSample->GetDim().Y;
				FUpdateTextureRegion2D Region(0, 0, 0, 0, InSample->GetDim().X, Height);
				RHICmdList.UpdateTexture2D(SampleDestinationAlphaTexture, 0, Region, Stride, Data);
			}
			
			TShaderMapRef< FHardwareVideoDecodingVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FHardwareVideoDecodingYCoCgAlphaPS > PixelShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
				SampleDestinationTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationTexture)
					.SetFormat(PF_DXT5));
			FShaderResourceViewRHIRef SRVAlpha = RHICmdList.CreateShaderResourceView(
				SampleDestinationAlphaTexture, 
				FRHIViewDesc::CreateTextureSRV()
					.SetDimensionFromTexture(SampleDestinationAlphaTexture)
					.SetFormat(PF_BC4));

			const FHardwareVideoDecodingShaderParams VsParameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(SRV, InSample->IsOutputSrgb());
			const FHardwareVideoDecodingShaderParams PsParameters = FWmfMediaHardwareVideoDecodingShader::GetParameters(SRV, SRVAlpha, InSample->IsOutputSrgb());

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VsParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PsParameters);
		}

		RHICmdList.DrawPrimitive(0, 2, 1);
		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}

	return true;
}

#endif
