// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaHardwareVideoDecodingShaders.h"

IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingVS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("MainVS"), SF_Vertex)
IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingPS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("NV12ConvertPS"), SF_Pixel)
IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingPassThroughPS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("PassThroughPS"), SF_Pixel)
IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingY416PS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("Y416ConvertPS"), SF_Pixel)
IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingYCoCgPS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("YCoCgConvertPS"), SF_Pixel)
IMPLEMENT_SHADER_TYPE(, FHardwareVideoDecodingYCoCgAlphaPS, TEXT("/Plugin/WmfMedia/Private/MediaHardwareVideoDecoding.usf"), TEXT("YCoCgAlphaConvertPS"), SF_Pixel)
