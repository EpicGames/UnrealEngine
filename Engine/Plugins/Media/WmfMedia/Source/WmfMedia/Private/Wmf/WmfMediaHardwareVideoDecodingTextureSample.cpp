// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaHardwareVideoDecodingTextureSample.h"

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

ID3D11Texture2D* FWmfMediaHardwareVideoDecodingTextureSample::InitializeSourceTexture(const TRefCountPtr<ID3D11Device>& InD3D11Device, FMediaTimeStamp InTime, FTimespan InDuration, const FIntPoint& InDim, EPixelFormat InFormat, EMediaTextureSampleFormat InMediaTextureSampleFormat)
{
	Time = InTime;
	Dim = InDim;
	OutputDim = InDim;
	Duration = InDuration;
	SampleFormat = InMediaTextureSampleFormat;
	Format = InFormat;

	if (SourceTexture.IsValid())
	{
		D3D11_TEXTURE2D_DESC Desc;
		SourceTexture->GetDesc(&Desc);

		if (Desc.Width == Dim.X && Desc.Height == Dim.Y)
		{
			return SourceTexture;
		}
	}

	D3D11_TEXTURE2D_DESC TextureDesc;
	TextureDesc.Width = Dim.X;
	TextureDesc.Height = Dim.Y;
	TextureDesc.MipLevels = 1;
	TextureDesc.ArraySize = 1;
	TextureDesc.Format = (DXGI_FORMAT)GPixelFormats[InFormat].PlatformFormat;
	TextureDesc.SampleDesc.Count = 1;
	TextureDesc.SampleDesc.Quality = 0;
	TextureDesc.Usage = D3D11_USAGE_DEFAULT;
	TextureDesc.BindFlags = 0;
	TextureDesc.CPUAccessFlags = 0;
	TextureDesc.MiscFlags = D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX;

	SourceTexture.Reset();
	HRESULT res = InD3D11Device->CreateTexture2D(&TextureDesc, nullptr, &SourceTexture);
	if (res != S_OK)
	{
		UE_LOGF(LogWmfMedia, Warning, "InD3D11Device->CreateTexture2D() for media source texture failed. (HR=%x)", res);
	}

	D3D11Device = InD3D11Device;

	return SourceTexture;
}

#if !UE_SERVER
void FWmfMediaHardwareVideoDecodingTextureSample::ShutdownPoolable()
{
	FWmfMediaTextureSample::ShutdownPoolable();

	// Correctly release the keyed mutex when the sample is returned to the pool
	TComPtr<IDXGIResource> OtherResource(nullptr);
	if (SourceTexture)
	{
		SourceTexture->QueryInterface(__uuidof(IDXGIResource), (void**)&OtherResource);
	}

	if (OtherResource)
	{
		HANDLE SharedHandle = nullptr;
		if (OtherResource->GetSharedHandle(&SharedHandle) == S_OK)
		{
			TComPtr<ID3D11Resource> SharedResource;
			D3D11Device->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (void**)&SharedResource);
			if (SharedResource)
			{
				TComPtr<IDXGIKeyedMutex> KeyedMutex;
				OtherResource->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);

				if (KeyedMutex)
				{
					// Reset keyed mutex
					if (KeyedMutex->AcquireSync(1, 0) == S_OK)
					{
						// Texture was never read
						KeyedMutex->ReleaseSync(0);
					}
					else if (KeyedMutex->AcquireSync(2, 0) == S_OK)
					{
						// Texture was read at least once
						KeyedMutex->ReleaseSync(0);
					}
				}
			}
		}
	}

	AlphaFormat = PF_Unknown;
	bIsDestinationTextureSRGB = false;
}
#endif

#endif
