// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformAtomics.h"
#include "Misc/AssertionMacros.h"

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <unknwn.h>
#include <shlwapi.h>
THIRD_PARTY_INCLUDES_END

/**
 * Implements the state information for asynchronous reads of byte buffer sources.
 *
 * The @see FWmfByteStream class uses this class to store the state for asynchronous
 * read requests that are initiated with @see FWmfByteStream.BeginRead and completed
 * with @see FWmfByteStream.EndRead
 */
class FWmfMediaReadState
	: public IUnknown
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InReadBuffer The buffer that receives the read data.
	 * @param InReadBufferSize The size of the read buffer.
	 */
	FWmfMediaReadState(BYTE* InReadBuffer, ULONG InReadBufferSize)
		: BytesRead(0)
		, ReadBuffer(InReadBuffer)
		, ReadBufferSize(InReadBufferSize)
		, RefCount(0)
	{ }

public:

	/**
	 * Adds the specified number of bytes read.
	 *
	 * @param BytesToAdd The number of bytes to add.
	 */
	void AddBytesRead(uint64 BytesToAdd)
	{
		BytesRead += BytesToAdd;
	}

	/**
	 * Gets a pointer to the buffer being read from.
	 *
	 * @return The buffer.
	 */
	BYTE* GetReadBuffer() const
	{
		return ReadBuffer;
	}

	/**
	 * Gets the size of the buffer being read from.
	 *
	 * @return The size of the buffer (in bytes).
	 */
	ULONG GetReadBufferSize() const
	{
		return ReadBufferSize;
	}

	/**
	 * Gets the actual number of bytes read.
	 *
	 * @return Bytes read count.
	 */
	uint64 GetBytesRead() const
	{
		return BytesRead;
	}

public:

	//~ IUnknown interface

#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

	STDMETHODIMP QueryInterface(REFIID RefID, void** Object)
	{
		static const QITAB QITab[] =
		{
			QITABENT(FWmfMediaReadState, IUnknown),
			{ 0 }
		};

		return QISearch( this, QITab, RefID, Object );
	}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif

	STDMETHODIMP_(ULONG) AddRef()
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	STDMETHODIMP_(ULONG) Release()
	{
		int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);
	
		if (CurrentRefCount == 0)
		{
			delete this;
		}

		return CurrentRefCount;
	}

private:

	/** Hidden destructor (this class is reference counted). */
	virtual ~FWmfMediaReadState()
	{
		check(RefCount == 0);
	}

private:

	/** Number of bytes read. */
	uint64 BytesRead;

	/** The buffer that receives the read data. */
	BYTE* ReadBuffer;

	/** The size of the read buffer. */
	ULONG ReadBufferSize;

	/** Holds a reference counter for this instance. */
	int32 RefCount;
};


#include "Windows/HideWindowsPlatformTypes.h"
