// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/Timespan.h"

#include "Windows/AllowWindowsPlatformTypes.h"


/**
 * Clock sink events.
 */
enum class EWmfMediaSamplerClockEvent
{
	Paused,
	Restarted,
	Started,
	Stopped
};


/**
 * Implements a callback object for the WMF sample grabber sink.
 */
class FWmfMediaSampler
	: public IMFSampleGrabberSinkCallback
{
public:

	/** Default constructor. */
	FWmfMediaSampler()
		: RefCount(0)
	{ }

public:

	/** Get an event that gets fired when the sampler's presentation clock changed its state. */
	DECLARE_EVENT_OneParam(FWmfMediaSampler, FOnClock, EWmfMediaSamplerClockEvent /*Event*/)
	FOnClock& OnClock()
	{
		return ClockEvent;
	}

	/** Get an event that gets fired when a new sample is ready (handler must be thread-safe). */
	DECLARE_EVENT_FourParams(FWmfMediaSampler, FOnSample, const uint8* /*Buffer*/, uint32 /*Size*/, FTimespan /*Duration*/, FTimespan /*Time*/)
	FOnSample& OnSample()
	{
		return SampleEvent;
	}

public:

	//~ IMFSampleGrabberSinkCallback interface

	STDMETHODIMP OnClockPause(MFTIME SystemTime)
	{
		ClockEvent.Broadcast(EWmfMediaSamplerClockEvent::Paused);

		return S_OK;
	}

	STDMETHODIMP OnClockRestart(MFTIME SystemTime)
	{
		ClockEvent.Broadcast(EWmfMediaSamplerClockEvent::Restarted);

		return S_OK;
	}

	STDMETHODIMP OnClockSetRate(MFTIME SystemTime, float flRate)
	{
		return S_OK;
	}

	STDMETHODIMP OnClockStart(MFTIME SystemTime, LONGLONG llClockStartOffset)
	{
		ClockEvent.Broadcast(EWmfMediaSamplerClockEvent::Started);

		return S_OK;
	}

	STDMETHODIMP OnClockStop(MFTIME SystemTime)
	{
		ClockEvent.Broadcast(EWmfMediaSamplerClockEvent::Stopped);

		return S_OK;
	}

	STDMETHODIMP OnProcessSample(REFGUID MajorMediaType, DWORD SampleFlags, LONGLONG SampleTime, LONGLONG SampleDuration, const BYTE* SampleBuffer, DWORD SampleSize)
	{
		SampleEvent.Broadcast((uint8*)SampleBuffer, (uint32)SampleSize, FTimespan(SampleDuration), FTimespan(SampleTime));

		return S_OK;
	}

	STDMETHODIMP OnSetPresentationClock(IMFPresentationClock* Clock)
	{
		return S_OK;
	}

	STDMETHODIMP OnShutdown()
	{
		return S_OK;
	}

public:

	//~ IUnknown interface

	STDMETHODIMP_(ULONG) AddRef()
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

	STDMETHODIMP QueryInterface(REFIID RefID, void** Object)
	{
		static const QITAB QITab[] =
		{
			QITABENT(FWmfMediaSampler, IMFSampleGrabberSinkCallback),
			{ 0 }
		};

		return QISearch(this, QITab, RefID, Object);
	}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif

	STDMETHODIMP_(ULONG) Release()
	{
		int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);

		if (CurrentRefCount == 0)
		{
			delete this;
		}

		return CurrentRefCount;
	}

private:

	/** Hidden destructor (this class is reference counted). */
	virtual ~FWmfMediaSampler()
	{
		check(RefCount == 0);
	}

private:

	/** Event that gets fired when the sampler's presentation clock changed state. */
	FOnClock ClockEvent;

	/** Holds a reference counter for this instance. */
	int32 RefCount;

	/** Event that gets fired when a new sample is ready. */
	FOnSample SampleEvent;
};


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
