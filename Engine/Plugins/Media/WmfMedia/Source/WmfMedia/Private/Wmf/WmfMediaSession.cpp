// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaSession.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "IMediaEventSink.h"
#include "Misc/ScopeLock.h"

#include "Player/WmfMediaTracks.h"
#include "WmfMediaUtils.h"

#include "Windows/AllowWindowsPlatformTypes.h"


#define WMFMEDIASESSION_USE_WINDOWS7FASTFORWARDENDHACK 1


/* Local helpers
 *****************************************************************************/

namespace WmfMediaSession
{
	/** Time span value for RequestedTime indicating a seek to the current time. */
	const FTimespan RequestedTimeCurrent = FTimespan::MaxValue();

	/** Get the human readable string representation of a media player state. */
	const TCHAR* StateToString(EMediaState State)
	{
		switch (State)
		{
		case EMediaState::Closed: return TEXT("Closed");
		case EMediaState::Error: return TEXT("Error");
		case EMediaState::Paused: return TEXT("Paused");
		case EMediaState::Playing: return TEXT("Playing");
		case EMediaState::Preparing: return TEXT("Preparing");
		case EMediaState::Stopped: return TEXT("Stopped");
		default: return TEXT("Unknown");
		}
	}
}


/* FWmfMediaSession structors
 *****************************************************************************/

FWmfMediaSession::FWmfMediaSession()
	: CanScrub(false)
	, Capabilities(0)
	, CurrentDuration(FTimespan::Zero())
	, MediaSessionCloseEvent(nullptr)
	, LastTime(FTimespan::Zero())
	, RefCount(0)
	, bIsRequestedTimeLoop(false)
	, bIsRequestedTimeSeek(false)
	, LastSetRate(-1.0f)
	, SessionRate(0.0f)
	, SessionState(EMediaState::Closed)
	, ShouldLoop(false)
	, Status(EMediaStatus::None)
	, bShuttingDown(false)
	, bIsWaitingForEnd(false)
	, bSeekActive(false)
	, NumSeeksInWorkQueue(0)
{
	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Created", this);
	MediaSessionCloseEvent = FPlatformProcess::GetSynchEventFromPool();
	check(MediaSessionCloseEvent != nullptr);
}


FWmfMediaSession::~FWmfMediaSession()
{
	check(RefCount == 0);
	Shutdown();

	FPlatformProcess::ReturnSynchEventToPool(MediaSessionCloseEvent);
	MediaSessionCloseEvent = nullptr;

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Destroyed", this);
}


/* FWmfMediaSession interface
 *****************************************************************************/

void FWmfMediaSession::GetEvents(TArray<EMediaEvent>& OutEvents)
{
	// Are we waiting for the end (of non-looping playback)?
	if (bIsWaitingForEnd)
	{
		/*
			Realistically we cannot check the tracks to have delivered all samples, because
			tracks may have different durations. If audio is shorter then the facade will not
			pull the remaining video samples. Also, WmfMedia delivers samples outside the movie's
			duration (with mp4/mov the duration is reported as specified in the movie box, but
			the tracks and the delivered samples may be longer, so we end up getting extra
			samples at the end that are outside the duration and will not be consumed, thus
			sticking around in the sample queues that will not empty.
		*/
#if 0
		// Is Tracks done with all its samples?
		TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
		if (TracksPinned.IsValid())
		{
			FMediaTimeStamp TimeStamp;
			bool bHaveSamples = TracksPinned->PeekVideoSampleTime(TimeStamp);
			if (bHaveSamples == false)
			{
				bIsWaitingForEnd = false;
				TracksPinned->SetSessionState(GetState());
			}
		}
		else
		{
			bIsWaitingForEnd = false;
		}

		// Are we done yet?
		if (bIsWaitingForEnd == false)
		{
			DeferredEvents.Enqueue(EMediaEvent::PlaybackEndReached);
		}
#else
		bIsWaitingForEnd = false;
		DeferredEvents.Enqueue(EMediaEvent::PlaybackEndReached);
#endif
	}

	// Hand out the events...
	EMediaEvent Event;

	while (DeferredEvents.Dequeue(Event))
	{
		OutEvents.Add(Event);
	}
}


void FWmfMediaSession::SetTracks(TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> InTracks)
{
	Tracks = InTracks;
}


bool FWmfMediaSession::Initialize(bool LowLatency)
{
	Shutdown();

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Initializing (LowLatency: %d)", this, LowLatency);

	// create session attributes
	TComPtr<IMFAttributes> Attributes;
	{
		HRESULT Result = ::MFCreateAttributes(&Attributes, 2);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to create media session attributes: %ls", this, *WmfMedia::ResultToString(Result));
			return false;
		}
	}

	if (LowLatency)
	{
		if (FPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			HRESULT Result = Attributes->SetUINT32(MF_LOW_LATENCY, TRUE);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to set low latency session attribute: %ls", this, *WmfMedia::ResultToString(Result));
			}
		}
		else
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Low latency media processing requires Windows 8 or newer", this);
		}
	}

	FScopeLock Lock(&CriticalSection);

	// create media session
	HRESULT Result = ::MFCreateMediaSession(Attributes, &MediaSession);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to create media session: %ls", this, *WmfMedia::ResultToString(Result));
		return false;
	}

	// start media event processing
	Result = MediaSession->BeginGetEvent(this, NULL);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start media session event processing: %ls", this, *WmfMedia::ResultToString(Result));
		return false;
	}

	SessionState = EMediaState::Preparing;

	return true;
}


void FWmfMediaSession::EnqueueWorkItem(FWorkItem&& WorkItem)
{
	FScopeLock Lock(&CriticalSection);

	// Anything in flight?
	if (CurrentWorkItem.IsValid())
	{
		// Just enqueue the item, we'll see an async notification and then can pump the queue...
		WorkItemQueue.Push(MoveTemp(WorkItem));
	}
	else
	{
		// Nothing in flight, so we need to execute in place & react to return codes appropriately...
		switch (WorkItem.WorkLoad(nullptr, MEUnknown, S_OK, WorkItem.State.Get()))
		{
			case EWorkItemResult::Done:			break;
			case EWorkItemResult::RetryLater:	CurrentWorkItem = WorkItem; break;
		}
	}
}


void FWmfMediaSession::JamWorkItem(FWorkItem&& WorkItem)
{
	FScopeLock Lock(&CriticalSection);

	// Anything in flight?
	if (CurrentWorkItem.IsValid())
	{
		// Just enqueue the item, we'll see an async notification and then can pump the queue...
		WorkItemQueue.Insert(MoveTemp(WorkItem), 0);
	}
	else
	{
		// Nothing in flight, so we need to execute in place & react to return codes appropriately...
		switch (WorkItem.WorkLoad(nullptr, MEUnknown, S_OK, WorkItem.State.Get()))
		{
			case EWorkItemResult::Done:			break;
			case EWorkItemResult::RetryLater:	CurrentWorkItem = WorkItem; break;
		}
	}
}


void FWmfMediaSession::ExecuteNextWorkItems(TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus)
{
	FScopeLock Lock(&CriticalSection);

	do
	{
		if (!CurrentWorkItem.IsValid())
		{
			if (WorkItemQueue.IsEmpty())
			{
				break;
			}

			CurrentWorkItem = WorkItemQueue[0];
			WorkItemQueue.RemoveAt(0);
		}

		switch (CurrentWorkItem.WorkLoad(MediaEvent, EventType, EventStatus, CurrentWorkItem.State.Get()))
		{
			case EWorkItemResult::Done:			CurrentWorkItem.Reset(); break;	// No new async job, try next workload
			case EWorkItemResult::RetryLater:	break;							// Workload should be retried later, done for now (implies that some async event or job is ensured to arrive to continue)
		}
	}
	while (!CurrentWorkItem.IsValid());
}


void FWmfMediaSession::ResetWorkItemQueue()
{
	WorkItemQueue.Empty();
	CurrentWorkItem.Reset();
	NumSeeksInWorkQueue = 0;
}


bool FWmfMediaSession::SetTopology(const TComPtr<IMFTopology>& InTopology, FTimespan InDuration)
{
	if (MediaSession == nullptr)
	{
		return false;
	}

	EnqueueWorkItem(FWorkItem([this, InTopology, InDuration](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session: %p: Setting new partial topology %p (duration = %ls)", this, InTopology.Get(), *InDuration.ToString());

			EWorkItemResult Res = EWorkItemResult::Done;

			if (SessionState == EMediaState::Preparing)
			{
				// media source resolved
				if (InTopology != nullptr)
				{
					// at least one track selected
					HRESULT Result = MediaSession->SetTopology(MFSESSION_SETTOPOLOGY_IMMEDIATE, InTopology);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to set partial topology %p: %ls", this, InTopology.Get(), *WmfMedia::ResultToString(Result));

						SessionState = EMediaState::Error;
						DeferredEvents.Enqueue(EMediaEvent::MediaOpenFailed);
					}
					else
					{
						// do nothing (Preparing state will exit in MESessionTopologyStatus event)
						Res = EWorkItemResult::Done;
					}
				}
				else
				{
					// no tracks selected
					UpdateCharacteristics();
					SessionState = EMediaState::Stopped;
					DeferredEvents.Enqueue(EMediaEvent::MediaOpened);
				}
			}
			else
			{
				// topology changed during playback, i.e. track switching
				Res = CommitTopology(InTopology, static_cast<FTopologyWorkItemState*>(BaseState), EventType);
			}

			if (Res != EWorkItemResult::RetryLater)
			{
				CurrentDuration = InDuration;
			}

			return Res;
		}, MakeShared<FTopologyWorkItemState>()));

	return true;
}


void FWmfMediaSession::Shutdown()
{
	if (MediaSession == NULL)
	{
		return;
	}

	UE_LOGF(LogWmfMedia, Verbose, "Session: %p: Shutting down", this);

	{
		// Scope needed since MediaSession->Close() cannot be locked, see below.
		FScopeLock Lock(&CriticalSection);
		ResetWorkItemQueue();
		bShuttingDown = true;
	}

	// When an error occurs we close the MediaSession in HandleError(), no need to Close it again.
	if (SessionState != EMediaState::Error)
	{
		// We cannot have a lock here since we are waiting for an event which
		// will call FWmfMediaSession::Invoke and would cause a deadlock

		HRESULT hr = MediaSession->Close();
		if (hr == S_OK)
		{
			// Wait for close event since Close is asynchronous
			MediaSessionCloseEvent->Wait();

			UE_LOGF(LogWmfMedia, Verbose, "Session: %p: Close", this);
		}
		else
		{
			UE_LOGF(LogWmfMedia, Warning, "Session: %p: Unable to close", this);
		}
	}

	FScopeLock Lock(&CriticalSection);

	MediaSession->Shutdown();
	MediaSession.Reset();

	CurrentTopology.Reset();
	PresentationClock.Reset();
	RateControl.Reset();
	RateSupport.Reset();

	CanScrub = false;
	Capabilities = 0;
	CurrentDuration = FTimespan::Zero();
	LastSetRate = -1.0f;
	SessionRate = 0.0f;
	UnpausedSessionRate = 0.0f;
	SessionState = EMediaState::Closed;
	LastTime = FTimespan::Zero();
	RequestedRate.Reset();
	Status = EMediaStatus::None;
	ThinnedRates.Empty();
	UnthinnedRates.Empty();

	bSeekActive = false;

	bShuttingDown = false;
}


/* IMediaControls interface
 *****************************************************************************/

bool FWmfMediaSession::CanControl(EMediaControl Control) const
{
	if (MediaSession == NULL)
	{
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	if (Control == EMediaControl::Pause)
	{
		return ((SessionState == EMediaState::Playing) && (((Capabilities & MFSESSIONCAP_PAUSE) != 0) || UnthinnedRates.Contains(0.0f)));
	}

	if (Control == EMediaControl::Resume)
	{
		return ((SessionState != EMediaState::Playing) && UnthinnedRates.Contains(1.0f));
	}

	if (Control == EMediaControl::Scrub)
	{
		return CanScrub;
	}

	if (Control == EMediaControl::Seek)
	{
		return (((Capabilities & MFSESSIONCAP_SEEK) != 0) && (CurrentDuration > FTimespan::Zero()));
	}

	return false;
}


FTimespan FWmfMediaSession::GetDuration() const
{
	FScopeLock Lock(&CriticalSection);
	return CurrentDuration;
}


float FWmfMediaSession::GetRate() const
{
	FScopeLock Lock(&CriticalSection);
	return (SessionState == EMediaState::Playing) ? SessionRate : 0.0f;
}


EMediaState FWmfMediaSession::GetState() const
{
	FScopeLock Lock(&CriticalSection);
	if ((SessionState == EMediaState::Playing) && (SessionRate == 0.0f))
	{
		return EMediaState::Paused;
	}

	// If we are waiting for the end then pretend that we are still playing.
	if (bIsWaitingForEnd)
	{
		return EMediaState::Playing;
	}

	return SessionState;
}


EMediaStatus FWmfMediaSession::GetStatus() const
{
	FScopeLock Lock(&CriticalSection);
	return Status;
}


TRangeSet<float> FWmfMediaSession::GetSupportedRates(EMediaRateThinning Thinning) const
{
	FScopeLock Lock(&CriticalSection);
	return (Thinning == EMediaRateThinning::Thinned) ? ThinnedRates : UnthinnedRates;
}


FTimespan FWmfMediaSession::GetTime() const
{
	/*
	* Note: for a V2 player this method has not a lot of meaning anymore - The PlayerFacade will generate timing information
	*/

	FScopeLock Lock(&CriticalSection);

	if (bShuttingDown)
	{
		// Exit, as we could otherwise block close-event delivery while being blocked on PresentationClock calls below & keeping the CS locked
		return FTimespan::Zero();
	}

	MFCLOCK_STATE ClockState;

	if (!PresentationClock.IsValid() || FAILED(PresentationClock->GetState(0, &ClockState)) || (ClockState == MFCLOCK_STATE_INVALID))
	{
		return FTimespan::Zero(); // topology not initialized, or clock not started yet
	}

	if (ClockState == MFCLOCK_STATE_STOPPED)
	{
		return LastTime; // WMF always reports zero when stopped
	}

	MFTIME ClockTime;
	MFTIME SystemTime;

	if (FAILED(PresentationClock->GetCorrelatedTime(0, &ClockTime, &SystemTime)))
	{
		return FTimespan::Zero();
	}

	return FTimespan(ClockTime);
}


bool FWmfMediaSession::IsLooping() const
{
	FScopeLock Lock(&CriticalSection);
	return ShouldLoop;
}


bool FWmfMediaSession::Seek(const FTimespan& Time, const FMediaSeekParams& InAdditionalParams)
{
	if (MediaSession == nullptr)
	{
		return false;
	}

	check(InAdditionalParams.NewSequenceIndex.IsSet());
	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Seeking to %ls with sequence index %d", this, *Time.ToString(), InAdditionalParams.NewSequenceIndex.GetValue());

	// validate seek
	if (!CanControl(EMediaControl::Seek))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Media source doesn't support seeking", this);
		return false;
	}

	if ((Time < FTimespan::Zero()) || (Time > CurrentDuration))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Invalid seek time %ls (media duration is %ls)", this, *Time.ToString(), *CurrentDuration.ToString());
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	if ((SessionState == EMediaState::Closed) || (SessionState == EMediaState::Error))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Cannot seek while closed or in error state", this);
		return false;
	}

	++NumSeeksInWorkQueue;
	NextSequenceIndex = InAdditionalParams.NewSequenceIndex;
	EnqueueWorkItem(FWorkItem([this, Time](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
		{
			auto State = static_cast<FTimeWorkItemState*>(BaseState);

			/*
			* We do not redo or cancel this due to a MESessionEnded being received, as triggering a "start" during this phase will
			* indeed start the session and interrupt the end of session events (including keeping requests alive etc.)
			*/

			if (State->State == FTimeWorkItemState::EState::WaitDone)
			{
				return (EventType == MESessionStarted) ? EWorkItemResult::Done : EWorkItemResult::RetryLater;
			}

			check(NumSeeksInWorkQueue > 0);
			if (--NumSeeksInWorkQueue == 0)
			{
				if (CommitTime(Time, true))
				{
					State->State = FTimeWorkItemState::EState::WaitDone;
					return EWorkItemResult::RetryLater;
				}
			}

			return EWorkItemResult::Done;
		}, MakeShared<FTimeWorkItemState, ESPMode::ThreadSafe>()));

	return true;
}


bool FWmfMediaSession::SetLooping(bool Looping)
{
	FScopeLock Lock(&CriticalSection);
	ShouldLoop = Looping;
	return true;
}


bool FWmfMediaSession::SetRate(float Rate)
{
	if (MediaSession == NULL)
	{
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	// validate rate
	if (!ThinnedRates.Contains(Rate) && !UnthinnedRates.Contains(Rate))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: The rate %f is not supported", this, Rate);
		return false;
	}

	if (LastSetRate == Rate)
	{
		return true;
	}

	LastSetRate = Rate;

	EnqueueWorkItem(FWorkItem([this, Rate](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Setting rate to %f", this, Rate);

			return CommitRate(Rate, static_cast<FRateWorkItemState*>(BaseState), EventType);
		}, MakeShared<FRateWorkItemState>()));

	return true;
}


/* IMFAsyncCallback interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FWmfMediaSession::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


STDMETHODIMP FWmfMediaSession::GetParameters(unsigned long*, unsigned long*)
{
	return E_NOTIMPL; // default behavior
}


STDMETHODIMP FWmfMediaSession::Invoke(IMFAsyncResult* AsyncResult)
{
	FScopeLock Lock(&CriticalSection);

	if (MediaSession == nullptr)
	{
		return S_OK;
	}

	// get event
	TComPtr<IMFMediaEvent> Event;
	{
		const HRESULT Result = MediaSession->EndGetEvent(AsyncResult, &Event);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get event: %ls", this, *WmfMedia::ResultToString(Result));
			return S_OK;
		}
	}

	MediaEventType EventType = MEUnknown;
	{
		const HRESULT Result = Event->GetType(&EventType);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get session event type: %ls", this, *WmfMedia::ResultToString(Result));
			return S_OK;
		}
	}

	HRESULT EventStatus = S_FALSE;
	{
		const HRESULT Result = Event->GetStatus(&EventStatus);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get event status: %ls", this, *WmfMedia::ResultToString(Result));
		}
	}

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Event [%ls]: %ls", this, *WmfMedia::MediaEventToString(EventType), *WmfMedia::ResultToString(EventStatus));

	// process event
	switch (EventType)
	{
	case MEBufferingStarted:
		Status |= EMediaStatus::Buffering;
		DeferredEvents.Enqueue(EMediaEvent::MediaBuffering);
		break;

	case MEBufferingStopped:
		Status &= ~EMediaStatus::Buffering;
		DeferredEvents.Enqueue(EMediaEvent::MediaBufferingComplete);
		break;

	case MEError:
		HandleError(EventStatus);
		break;

	case MEReconnectEnd:
		Status &= ~EMediaStatus::Connecting;
		break;

	case MEReconnectStart:
		Status |= EMediaStatus::Connecting;
		DeferredEvents.Enqueue(EMediaEvent::MediaConnecting);
		break;

	case MESessionCapabilitiesChanged:
		Capabilities = ::MFGetAttributeUINT32(Event, MF_EVENT_SESSIONCAPS, Capabilities);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionClosed:
		MediaSessionCloseEvent->Trigger();
		Capabilities = 0;
		LastTime = FTimespan::Zero();
		ResetWorkItemQueue();
		break;

	case MEEndOfPresentation:
		HandlePresentationEnded();
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionEnded:
		HandleSessionEnded();
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionPaused:
		HandleSessionPaused(EventStatus);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionRateChanged:
		HandleSessionRateChanged(EventStatus, *Event);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionScrubSampleComplete:
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionStarted:
		HandleSessionStarted(EventStatus);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionStopped:
		HandleSessionStopped(EventStatus);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionTopologySet:
		HandleSessionTopologySet(EventStatus, *Event);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	case MESessionTopologyStatus:
		HandleSessionTopologyStatus(EventStatus, *Event);
		ExecuteNextWorkItems(Event, EventType, EventStatus);
		break;

	default:
		break; // unsupported event
	}

	// request next event
	if ((EventType != MESessionClosed) && (SessionState != EMediaState::Error) && (MediaSession != NULL))
	{
		const HRESULT Result = MediaSession->BeginGetEvent(this, NULL);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to request next session event; aborting playback: %ls", this, *WmfMedia::ResultToString(Result));

			Capabilities = 0;
			SessionState = EMediaState::Error;
		}
	}

	// Tell the tracks about our state.
	TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
	if (TracksPinned.IsValid())
	{
		TracksPinned->SetSessionState(GetState());
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "Session %p: CurrentState: %ls, CurrentRate: %f, CurrentTime: %ls, SessionState: %ls, SessionRate: %f",
		this,
		WmfMediaSession::StateToString(GetState()),
		GetRate(),
		*GetTime().ToString(),
		WmfMediaSession::StateToString(SessionState),
		SessionRate
	);

	UE_LOGF(LogWmfMedia, VeryVerbose, "Session %p: WaitingForEnd:%d",
		this,
		bIsWaitingForEnd);

	return S_OK;
}


#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif // _MSC_VER == 1900

STDMETHODIMP FWmfMediaSession::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FWmfMediaSession, IMFAsyncCallback),
		{ 0 }
	};

	return QISearch(this, QITab, RefID, Object);
}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif // _MSC_VER == 1900


STDMETHODIMP_(ULONG) FWmfMediaSession::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);

	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


/* FWmfMediaSession implementation
 *****************************************************************************/

FWmfMediaSession::EWorkItemResult FWmfMediaSession::CommitRate(float Rate, FRateWorkItemState* State, MediaEventType EventType)
{
	check(MediaSession != nullptr);

	UE_LOGF(LogWmfMedia, VeryVerbose, "Session %p: Committing rate %f [S=%d]", this, Rate, int(State->State));

	// Did session end and we do not just start up?
	if (EventType == MESessionEnded && State->State != FRateWorkItemState::EState::Begin)
	{
		// Make us start over again... what we did so far has been lost in the deep belly of WMF's machinery...
		State->State = FRateWorkItemState::EState::Begin;
	}

	if (RateControl == nullptr)
	{
		/*
		* No rate control
		*
		* Session is only able to support pause & play
		* (no reverse, scrubbing or speed changes)
		*/

		if (State->State == FRateWorkItemState::EState::Begin)
		{
			if (Rate == 0.0f)
			{
				if (SessionState == EMediaState::Playing)
				{
					const HRESULT Result = MediaSession->Pause();

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to pause session: %ls", this, *WmfMedia::ResultToString(Result));
						return EWorkItemResult::Done;
					}

					State->State = FRateWorkItemState::EState::WaitForPause;
					return EWorkItemResult::RetryLater;
				}
			}
			else
			{
				if (SessionState != EMediaState::Playing)
				{
					PROPVARIANT StartPosition;
					PropVariantInit(&StartPosition);

					const HRESULT Result = MediaSession->Start(NULL, &StartPosition);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start session: %ls", this, *WmfMedia::ResultToString(Result));
						return EWorkItemResult::Done;
					}

					State->State = FRateWorkItemState::EState::WaitForStart;
					return EWorkItemResult::RetryLater;
				}
			}
		}
		else if (State->State == FRateWorkItemState::EState::WaitForPause)
		{
			if (EventType != MESessionPaused)
			{
				return EWorkItemResult::RetryLater;
			}
		}
		else
		{
			check(State->State == FRateWorkItemState::EState::WaitForStart);

			if (EventType != MESessionStarted)
			{
				return EWorkItemResult::RetryLater;
			}
		}

		return EWorkItemResult::Done;
	}

	// ---------------------------------------------------------------------------------------------------------------
	/**
	 * Rate control is present
	 *
	 * Session can plat at variable speeds, reverse (possibly) and scrub (possibly)
	 */

	if (State->State == FRateWorkItemState::EState::Begin)
	{
		State->State = FRateWorkItemState::EState::Ready;
		State->LastTime = WmfMediaSession::RequestedTimeCurrent;

		if (Rate != 0.0f && ((UnpausedSessionRate * Rate) < 0.0f))
		{
			/**
			 * Reversal of playback direction
			 *
			 * System needs to be stopped and we ensure no pending requests as WMF tends to get confused if they come in while we do this
			 */

			// Stopped already?
			if (SessionState != EMediaState::Stopped)
			{
				// No...

				// If we reverse, we might loose issued sample requests (without us knowing) - wait until all current ones are done
				TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
				if (TracksPinned.IsValid())
				{
					// Schedule the lambda to be executed once all requests are done if any are pending...
					if (TracksPinned->ExecuteOnceMediaStreamSinkHasNoPendingRequests([this]()
						{
							// If we have pending sample requests, this will be executed by the sink once it detects all of them being done.
							// All we do is to pump our workitem queue once more to ensure that - no matter what internal async wakeups we get we at least get this one to continue!
							FScopeLock Lock(&CriticalSection);
							ExecuteNextWorkItems(nullptr, MEUnknown, S_OK);
						}))
					{
						// We had pending requests. Reset to original state and a later retry
						State->State = FRateWorkItemState::EState::Begin;
						return EWorkItemResult::RetryLater;
					}
				}

				// Gather current clock, so we can restart at the same spot after the stop/reverse step
				LONGLONG ClockTime;
				MFTIME SystemTime;
				if (PresentationClock->GetCorrelatedTime(0, &ClockTime, &SystemTime) == S_OK)
				{
					State->LastTime = FTimespan(ClockTime);
				}

				// Stop the media session
				const HRESULT Result = MediaSession->Stop();
				if (FAILED(Result))
				{
					UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to stop for rate change: %ls", this, *WmfMedia::ResultToString(Result));
					return EWorkItemResult::Done;
				}

				// Next wait for this to finish and then set & restart...
				State->State = FRateWorkItemState::EState::WaitForStopThenSetAndRestart;
				return EWorkItemResult::RetryLater;
			}

			// We are stopped. Just continue to the rate set...
		}
		else
		{
			if (Rate != 0.0f)
			{
				if (SessionState != EMediaState::Playing)
				{

					// Going out of "paused" state, but we want to actually we have been "scrubbing", which WMF interprets as a PLAYING state!
					// So we need to pause first, to then restart normal playback (or else the rate change fails)
					// [this also triggers on initial startup, but it seems harmless to pause for no reason]

					if (SessionState != EMediaState::Stopped)
					{
						/**
						 * Leave paused state
						 */

						if (CanScrub)
						{
							/*
							* As we can scrub, we actually are leaving "scrub state", not a real pause, but a playback state as far as WMF is concerned!
							*/

							// Was the last non-scrubbing playback reverse?
							if (UnpausedSessionRate >= 0.0f)
							{
								// No. We need to pause to be able to set a new non-zero rate...
								const HRESULT Result = MediaSession->Pause();
								if (FAILED(Result))
								{
									UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to pause for rate change: %ls", this, *WmfMedia::ResultToString(Result));
									return EWorkItemResult::Done;
								}

								// Wait for pause to be done, then set and restart...
								State->State = FRateWorkItemState::EState::WaitForPauseThenSetAndRestart;
							}
							else
							{
								// Yes. We need to actually stop session as we are going to go back into a reverse playback situation from scrubbing

								// Get the time so we can restart at the correct spot
								LONGLONG ClockTime;
								MFTIME SystemTime;
								if (PresentationClock->GetCorrelatedTime(0, &ClockTime, &SystemTime) == S_OK)
								{
									State->LastTime = FTimespan(ClockTime);
								}

								// Stop the session
								const HRESULT Result = MediaSession->Stop();
								if (FAILED(Result))
								{
									UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to stop for rate change: %ls", this, *WmfMedia::ResultToString(Result));
									return EWorkItemResult::Done;
								}

								// Wait for pause to be done, then set and restart...
								State->State = FRateWorkItemState::EState::WaitForStopThenSetAndRestart;
							}
						}
						else
						{
							/**
							 * No scrubbing. We paused "for real". No just start things up again...
							 */
							PROPVARIANT StartPosition;
							PropVariantInit(&StartPosition);

							const HRESULT Result = MediaSession->Start(NULL, &StartPosition);

							if (FAILED(Result))
							{
								UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start session: %ls", this, *WmfMedia::ResultToString(Result));
								return EWorkItemResult::Done;
							}

							State->State = FRateWorkItemState::EState::WaitForStart;
						}

						return EWorkItemResult::RetryLater;
					}

					// We are stopped. We can set a new rate and restart without any other work before that...
					State->State = FRateWorkItemState::EState::ReadyNeedsRestart;
				}
			}
			else
			{
				if (SessionState == EMediaState::Playing)
				{
					/**
					 * Entering pause state
					 */

					if (CanScrub)
					{
						/*
						* We can scrub. Hence we rather enter 'scrub mode' than a real 'pause' state
						*/

						// Wait for any pending requests as WMF tends to loose track of these during the transition...
						TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
						if (TracksPinned.IsValid())
						{
							// Schedule the lambda to be run once we have no outstanding requests anymore if we ever had any
							if (TracksPinned->ExecuteOnceMediaStreamSinkHasNoPendingRequests([this]()
								{
									// If we have pending sample requests, this will be executed by the sink once it detects all of them being done.
									// All we do is to pump our workitem queue once more to ensure that - no matter what internal async wakeups we get we at least get this one to continue!
									FScopeLock Lock(&CriticalSection);
									ExecuteNextWorkItems(nullptr, MEUnknown, S_OK);
								}))
							{
								// We had outstanding requests, so we reset state and retry later
								State->State = FRateWorkItemState::EState::Begin;
								return EWorkItemResult::RetryLater;
							}
						}
					}

					// Last unpaused rate was forward OR we cannot scrub...
					if (UnpausedSessionRate >= 0.0f || !CanScrub)
					{
						// Pause the session
						const HRESULT Result = MediaSession->Pause();
						if (FAILED(Result))
						{
							UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to pause for rate change: %ls", this, *WmfMedia::ResultToString(Result));
							return EWorkItemResult::Done;
						}

						// Continue to start up into scrub mode (start with rate == 0) or simply leave things plaused...
						State->State = CanScrub ? FRateWorkItemState::EState::WaitForPauseThenSetAndRestart : FRateWorkItemState::EState::WaitForPause;
					}
					else
					{
						// We had been in reverse. We need to stop the session to be able to go into 'srub mode'...

						// Get the current playback time so we can restart at the correct spot
						LONGLONG ClockTime;
						MFTIME SystemTime;
						if (PresentationClock->GetCorrelatedTime(0, &ClockTime, &SystemTime) == S_OK)
						{
							State->LastTime = FTimespan(ClockTime);
						}

						// Stop the session
						const HRESULT Result = MediaSession->Stop();
						if (FAILED(Result))
						{
							UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to stop for rate change: %ls", this, *WmfMedia::ResultToString(Result));
							return EWorkItemResult::Done;
						}

						// Wait for the stop and then continue with a restart into 'scrub mode'
						State->State = FRateWorkItemState::EState::WaitForStopThenSetAndRestart;
					}

					return EWorkItemResult::RetryLater;
				}

				// Just set the rate now. No restart...
			}
		}
	}
	else if (State->State == FRateWorkItemState::EState::WaitForStopThenSetAndRestart)
	{
		// Wait for stop and then set and restart...
		if (EventType != MESessionStopped)
		{
			return EWorkItemResult::RetryLater;
		}
		State->State = FRateWorkItemState::EState::ReadyNeedsRestart;
	}
	else if (State->State == FRateWorkItemState::EState::WaitForPauseThenSetAndRestart)
	{
		// Wait for pause and then set and restart...
		if (EventType != MESessionPaused)
		{
			return EWorkItemResult::RetryLater;
		}
		State->State = FRateWorkItemState::EState::ReadyNeedsRestart;
	}
	else if (State->State == FRateWorkItemState::EState::WaitForPause)
	{
		// Check if pause is all done!
		return (EventType == MESessionPaused) ? EWorkItemResult::Done : EWorkItemResult::RetryLater;
	}
	else if (State->State == FRateWorkItemState::EState::WaitForSetAndRestart)
	{
		// Wait for rate set and then restart...
		if (EventType != MESessionRateChanged)
		{
			return EWorkItemResult::RetryLater;
		}

		// Restart the session at the given time (if none is specified, we assume the 'current time' is valid with the session)
		PROPVARIANT StartPosition;
		PropVariantInit(&StartPosition);
		if (State->LastTime != WmfMediaSession::RequestedTimeCurrent)
		{
			StartPosition.vt = VT_I8;
			StartPosition.hVal.QuadPart = State->LastTime.GetTicks();
		}

		const HRESULT Result = MediaSession->Start(NULL, &StartPosition);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start session: %ls", this, *WmfMedia::ResultToString(Result));
			return EWorkItemResult::Done;
		}

		// Now wait for the restart to be done...
		State->State = FRateWorkItemState::EState::WaitForRestart;
		return EWorkItemResult::RetryLater;
	}
	else if (State->State == FRateWorkItemState::EState::WaitForSet)
	{
		// Wait for rate being set (no restart)
		if (EventType != MESessionRateChanged)
		{
			return EWorkItemResult::RetryLater;
		}
		return EWorkItemResult::Done;
	}
	else if (State->State == FRateWorkItemState::EState::WaitForRestart)
	{
		// Check if restart is all done!
		return (EventType == MESessionStarted) ? EWorkItemResult::Done : EWorkItemResult::RetryLater;
	}

	// determine thinning mode
	EMediaRateThinning Thinning;
	if (UnthinnedRates.Contains(Rate))
	{
		Thinning = EMediaRateThinning::Unthinned;
	}
	else if (ThinnedRates.Contains(Rate))
	{
		Thinning = EMediaRateThinning::Thinned;
	}
	else
	{
		return EWorkItemResult::Done;
	}

	// ...and set the rate
	{
		const HRESULT Result = RateControl->SetRate((Thinning == EMediaRateThinning::Thinned) ? TRUE : FALSE, Rate);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to change rate: %ls", this, *WmfMedia::ResultToString(Result));
			return EWorkItemResult::Done;
		}
	}

	// Now we wait for the rate to be set and possibly trigger a restart...
	State->State = (State->State == FRateWorkItemState::EState::ReadyNeedsRestart) ? FRateWorkItemState::EState::WaitForSetAndRestart : FRateWorkItemState::EState::WaitForSet;

	return EWorkItemResult::RetryLater;
}


bool FWmfMediaSession::CommitTime(FTimespan Time, bool bIsSeek)
{
	check(MediaSession != NULL);

	FTimespan OriginalTime = Time;
	const FString TimeString = (Time == WmfMediaSession::RequestedTimeCurrent) ? TEXT("<current>") : *Time.ToString();
	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Committing time %ls (Seek=%d)", this, *TimeString, bIsSeek);

	// start session at requested time
	PROPVARIANT StartPosition;
	const bool bCanSeek = CanControl(EMediaControl::Seek);
	if (!bCanSeek)
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Starting from <current>, because media can't seek", this);
		Time = WmfMediaSession::RequestedTimeCurrent;
	}

	if (Time == WmfMediaSession::RequestedTimeCurrent)
	{
		StartPosition.vt = VT_EMPTY; // current time
	}
	else
	{
		StartPosition.vt = VT_I8;
		StartPosition.hVal.QuadPart = Time.GetTicks();
	}

	HRESULT Result = MediaSession->Start(NULL, &StartPosition);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start session: %ls", this, *WmfMedia::ResultToString(Result));
		return false;
	}

	// If this is not a loop, then tell the tracks about the seek.
	if (bCanSeek && bIsSeek)
	{
		TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
		if (TracksPinned.IsValid())
		{
			check(NextSequenceIndex.IsSet());
			TracksPinned->SeekStarted(OriginalTime, NextSequenceIndex.GetValue());
			NextSequenceIndex.Reset();
		}
		bSeekActive = true;
	}

	return true;
}


FWmfMediaSession::EWorkItemResult FWmfMediaSession::CommitTopology(IMFTopology* Topology, FTopologyWorkItemState* State, MediaEventType EventType)
{
	check(MediaSession != NULL);

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Committing topology %p [S=%d]", this, Topology, int(State->State));

	// Did session end and we do not just start up?
	if (EventType == MESessionEnded && State->State != FTopologyWorkItemState::EState::Begin)
	{
		// Make us start over again... what we did so far has been lost in the deep belly of WMF's machinery...
		State->State = FTopologyWorkItemState::EState::Begin;
	}

	if (State->State == FTopologyWorkItemState::EState::Begin)
	{
		// Starting up for the first time. Check if we are stopped already?
		if (SessionState != EMediaState::Stopped)
		{
			// No. Stop session and retry...
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Stopping session for topology change", this);

			// topology change requires transition to Stopped; playback is resumed afterwards
			HRESULT Result = MediaSession->Stop();
			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to stop for topology change: %ls", this, *WmfMedia::ResultToString(Result));
				return EWorkItemResult::Done;
			}

			State->State = FTopologyWorkItemState::EState::ReadyNeedsRestart;

			// Come back later once the session has stopped...
			return EWorkItemResult::RetryLater;
		}
		else
		{
			// Proceed to actual setup...
			State->State = FTopologyWorkItemState::EState::Ready;
		}
	}
	else if (State->State == FTopologyWorkItemState::EState::Ready)
	{
		return (EventType != MESessionTopologySet) ? EWorkItemResult::RetryLater : EWorkItemResult::Done;
	}
	else if (State->State == FTopologyWorkItemState::EState::ReadyNeedsRestart)
	{
		if (EventType != MESessionTopologySet)
		{
			return EWorkItemResult::RetryLater;
		}

		// Topology is set, we need to restart the session...
		PROPVARIANT StartPosition;
		PropVariantInit(&StartPosition);
		HRESULT Result = MediaSession->Start(NULL, &StartPosition);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to start after topology change: %ls", this, *WmfMedia::ResultToString(Result));
			return EWorkItemResult::Done;
		}

		State->State = FTopologyWorkItemState::EState::WaitRestart;
		return EWorkItemResult::RetryLater;
	}
	else if (State->State == FTopologyWorkItemState::EState::WaitRestart)
	{
		return (EventType != MESessionStarted) ? EWorkItemResult::RetryLater : EWorkItemResult::Done;
	}

	// Actual setup...

	// Clear any topology enqueued to be set prior to the one we are about to set
	// (this is async, but we omit a specific state change and wait as we execute the next async command unconditonally right afterwards - and we wait for that one)
	HRESULT Result = MediaSession->ClearTopologies();
	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to clear queued topologies: %ls", this, *WmfMedia::ResultToString(Result));
		return EWorkItemResult::Done;
	}

	// Set or clear topology...
	if (Topology != nullptr)
	{
		Result = MediaSession->SetTopology(MFSESSION_SETTOPOLOGY_IMMEDIATE, Topology);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to set topology %p: %ls", Topology, this, *WmfMedia::ResultToString(Result));
			return EWorkItemResult::Done;
		}
	}
	else
	{
		Result = MediaSession->SetTopology(MFSESSION_SETTOPOLOGY_CLEAR_CURRENT, Topology);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to set topology %p: %ls", Topology, this, *WmfMedia::ResultToString(Result));
			return EWorkItemResult::Done;
		}
	}

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Committed topology %p", this, Topology);

	return EWorkItemResult::RetryLater;
}


void FWmfMediaSession::UpdateCharacteristics()
{
	// reset characteristics
	PresentationClock.Reset();
	RateControl.Reset();
	RateSupport.Reset();

	ThinnedRates.Empty();
	UnthinnedRates.Empty();

	CanScrub = false;

	if (MediaSession == NULL)
	{
		return;
	}

	// get presentation clock, if available
	TComPtr<IMFClock> Clock;

	HRESULT Result = MediaSession->GetClock(&Clock);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Session clock unavailable: %ls", this, *WmfMedia::ResultToString(Result));
	}
	else
	{
		Result = Clock->QueryInterface(IID_PPV_ARGS(&PresentationClock));

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Presentation clock unavailable: %ls", this, *WmfMedia::ResultToString(Result));
		}
		else
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Presentation clock ready", this);
		}
	}

	// get rate control & rate support, if available
	Result = ::MFGetService(MediaSession, MF_RATE_CONTROL_SERVICE, IID_PPV_ARGS(&RateControl));

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Rate control service unavailable: %ls", this, *WmfMedia::ResultToString(Result));
	}
	else
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Rate control ready", this);

		float NewRate;
		if (FAILED(RateControl->GetRate(FALSE, &NewRate)))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to initialize current rate", this);
			NewRate = 1.0f; // the session's initial play rate is usually 1.0
		}

		SetSessionRate(NewRate);
	}

	Result = ::MFGetService(MediaSession, MF_RATE_CONTROL_SERVICE, IID_PPV_ARGS(&RateSupport));

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Rate support service unavailable: %ls", this, *WmfMedia::ResultToString(Result));
	}
	else
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Rate support ready", this);
	}

	// cache rate control properties
	if (RateSupport.IsValid())
	{
		float MaxRate = 0.0f;
		float MinRate = 0.0f;

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_FORWARD, TRUE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_FORWARD, TRUE, &MaxRate)))
		{
			ThinnedRates.Add(TRange<float>::Inclusive(MinRate, MaxRate));
		}

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_REVERSE, TRUE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_REVERSE, TRUE, &MaxRate)))
		{
			ThinnedRates.Add(TRange<float>::Inclusive(MaxRate, MinRate));
		}

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_FORWARD, FALSE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_FORWARD, FALSE, &MaxRate)))
		{
			UnthinnedRates.Add(TRange<float>::Inclusive(MinRate, MaxRate));
		}

		if (SUCCEEDED(RateSupport->GetSlowestRate(MFRATE_REVERSE, FALSE, &MinRate)) &&
			SUCCEEDED(RateSupport->GetFastestRate(MFRATE_REVERSE, FALSE, &MaxRate)))
		{
			UnthinnedRates.Add(TRange<float>::Inclusive(MaxRate, MinRate));
		}

		// Check for scrubbing (playback at 0.0 speed)
		/*
		* We do NOT use IsRateSupported() as it seemns to produce incorrect results quite often (claiming incorrect support for 0.0, but advising 1.0 as best alternate rate)
		*/
		CanScrub = ThinnedRates.Contains(0.0f);

		// When native out is enabled, the slowest rate will be greater than 0.0f
		bool SupportsUnthinnedPause = SUCCEEDED(RateSupport->IsRateSupported(FALSE, 0.0f, NULL));
		if (SupportsUnthinnedPause && !ThinnedRates.Contains(0.0f) && !UnthinnedRates.Contains(0.0f))
		{
			UnthinnedRates.Add(TRange<float>::Inclusive(0.0f, 0.0f));
		}
	}
}


void FWmfMediaSession::SetSessionRate(float Rate)
{
	if (SessionRate != Rate)
	{
		SessionRate = Rate;
		if (Rate != 0.0f)
		{
			UnpausedSessionRate = Rate;
		}
	}
}


void::FWmfMediaSession::Flush()
{
	FScopeLock Lock(&CriticalSection);
	NextSequenceIndex.Reset();
}


void FWmfMediaSession::RequestMoreVideoData()
{
	EnqueueWorkItem(FWorkItem([this](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
		{
			TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
			if (TracksPinned.IsValid())
			{
				TracksPinned->RequestMoreVideoDataFromStreamSink();
			}
			return EWorkItemResult::Done;
		}));
}

/* FWmfMediaSession callbacks
*****************************************************************************/

void FWmfMediaSession::HandleError(HRESULT EventStatus)
{
	UE_LOGF(LogWmfMedia, Error, "An error occurred in the media session: %ls", *WmfMedia::ResultToString(EventStatus));

	SessionState = EMediaState::Error;
	MediaSession->Close();

	ResetWorkItemQueue();
}


void FWmfMediaSession::HandlePresentationEnded()
{
	SessionState = EMediaState::Stopped;
}


void FWmfMediaSession::HandleSessionEnded()
{
	UE_LOGF(LogWmfMedia, VeryVerbose, "FWmfMediaSession::HandleSessionEnded ShouldLoop:%d", ShouldLoop);

	SessionState = EMediaState::Stopped;

	TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
	if (TracksPinned.IsValid())
	{
		TracksPinned->SessionEnded();
	}

	// We only loop if we don't have a seek in progress right now
	if (!bSeekActive)
	{
		if (ShouldLoop)
		{
			// loop back to beginning/end
			JamWorkItem(FWorkItem([this](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
				{
					auto State = static_cast<FTimeWorkItemState*>(BaseState);

					if (State->State == FTimeWorkItemState::EState::WaitDone)
					{
						return (EventType == MESessionStarted) ? EWorkItemResult::Done : EWorkItemResult::RetryLater;
					}

					if (CommitTime((UnpausedSessionRate < 0.0f) ? CurrentDuration : FTimespan::Zero(), false))
					{
						TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> TracksPinned = Tracks.Pin();
						if (TracksPinned.IsValid())
						{
							TracksPinned->LoopStarted(UnpausedSessionRate);
						}

						State->State = FTimeWorkItemState::EState::WaitDone;
						return EWorkItemResult::RetryLater;
					}

					return EWorkItemResult::Done;
				}, MakeShared<FTimeWorkItemState, ESPMode::ThreadSafe>()));
		}
		else
		{
			LastTime = FTimespan::Zero();
			bIsWaitingForEnd = true;
			ResetWorkItemQueue();
		}
	}
}


void FWmfMediaSession::HandleSessionPaused(HRESULT EventStatus)
{
	if (RateControl == nullptr || !CanScrub)
	{
		DeferredEvents.Enqueue(EMediaEvent::PlaybackSuspended);
		SessionState = EMediaState::Paused;
	}
}


void FWmfMediaSession::HandleSessionRateChanged(HRESULT EventStatus, IMFMediaEvent& Event)
{
	if (SUCCEEDED(EventStatus))
	{
		// cache current rate
		PROPVARIANT Value;
		::PropVariantInit(&Value);

		const HRESULT Result = Event.GetValue(&Value);

		if (SUCCEEDED(Result) && (Value.vt == VT_R4))
		{
			SetSessionRate(Value.fltVal);
		}
	}
	else if (RateControl != NULL)
	{

		BOOL Thin = FALSE;
		float Rate = 0.0f; // quiet down SA validation
		if (RateControl->GetRate(&Thin, &Rate) != S_OK)
		{
			Rate = 1.0f;
		}
		SetSessionRate(Rate);
	}
}


void FWmfMediaSession::HandleSessionStarted(HRESULT EventStatus)
{
	if (SUCCEEDED(EventStatus))
	{
		bIsWaitingForEnd = false;

		if (RateControl == NULL)
		{
			SetSessionRate(1.0f);
		}
		else
		{
			float Rate = 0.0f; // quiet down SA validation
			if (RateControl->GetRate(NULL, &Rate) != S_OK)
			{
				Rate = 1.0f;
			}
			SetSessionRate(Rate);
		}

		if (SessionRate != 0.0f)
		{
			SessionState = EMediaState::Playing;
			DeferredEvents.Enqueue(EMediaEvent::PlaybackResumed);
		}
		else
		{
			// We are pausing (or "scrubbing" in WMF terms - hence this is a playback mode, not real "pause")
			SessionState = EMediaState::Paused;
			DeferredEvents.Enqueue(EMediaEvent::PlaybackSuspended);
		}

		UE_LOGF(LogWmfMedia, VeryVerbose, "Session - Started: R=%f SState=%d", SessionRate, int(SessionState));

		if (bSeekActive && (SessionState == EMediaState::Playing || SessionState == EMediaState::Paused))
		{
			DeferredEvents.Enqueue(EMediaEvent::SeekCompleted);
		}
		bSeekActive = false;
	}
}


void FWmfMediaSession::HandleSessionStopped(HRESULT EventStatus)
{
	if (SUCCEEDED(EventStatus))
	{
		SessionState = EMediaState::Stopped;
		DeferredEvents.Enqueue(EMediaEvent::PlaybackSuspended);
	}
}


void FWmfMediaSession::HandleSessionTopologySet(HRESULT EventStatus, IMFMediaEvent& Event)
{
	if (SUCCEEDED(EventStatus))
	{
		const HRESULT Result = WmfMedia::GetTopologyFromEvent(Event, CurrentTopology);

		if (SUCCEEDED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Topology %p set as current", this, CurrentTopology.Get());
			return;
		}

		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get topology that was set: %ls", this, *WmfMedia::ResultToString(Result));
	}

	if (SessionState == EMediaState::Preparing)
	{
		UE_LOGF(LogWmfMedia, Error, "An error occurred in the media session: %ls", *WmfMedia::ResultToString(EventStatus));

		// an error occurred in the topology
		SessionState = EMediaState::Error;
		DeferredEvents.Enqueue(EMediaEvent::MediaOpenFailed);
	}
}


void FWmfMediaSession::HandleSessionTopologyStatus(HRESULT EventStatus, IMFMediaEvent& Event)
{
	// get the status of the topology that generated the event
	MF_TOPOSTATUS TopologyStatus = MF_TOPOSTATUS_INVALID;
	{
		const HRESULT Result = Event.GetUINT32(MF_EVENT_TOPOLOGY_STATUS, (UINT32*)&TopologyStatus);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get topology status: %ls", this, *WmfMedia::ResultToString(Result));
			return;
		}
	}

	// get the topology that generated the event
	TComPtr<IMFTopology> Topology;
	{
		const HRESULT Result = WmfMedia::GetTopologyFromEvent(Event, Topology);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get topology from topology status event: %ls", this, *WmfMedia::ResultToString(Result));
			return;
		}
	}

	UE_LOGF(LogWmfMedia, Verbose, "Session %p: Topology %p changed status to %ls", this, Topology.Get(), *WmfMedia::TopologyStatusToString(TopologyStatus));

	// Note: the ordering of topology status events is not guaranteed for two
	// consecutive topologies. we skip events that are not the for current one.

	if (Topology != CurrentTopology)
	{
		return;
	}

	if (SessionState == EMediaState::Error)
	{
		ResetWorkItemQueue();
		return;
	}

	if (FAILED(EventStatus))
	{
		if (SessionState == EMediaState::Preparing)
		{
			UE_LOGF(LogWmfMedia, Error, "An error occured when preparing the topology");

			// an error occurred in the topology
			SessionState = EMediaState::Error;
			DeferredEvents.Enqueue(EMediaEvent::MediaOpenFailed);
		}

		ResetWorkItemQueue();
		return;
	}

	if (TopologyStatus != MF_TOPOSTATUS_READY)
	{
		return;
	}

#if 0
	TComPtr<IMFTopology> FullTopology;
	{
		const HRESULT Result = MediaSession->GetFullTopology(MFSESSION_GETFULLTOPOLOGY_CURRENT, 0, &FullTopology);

		if (SUCCEEDED(Result))
		{
			if (FullTopology != CurrentTopology)
			{
				UE_LOGF(LogWmfMedia, Verbose, "Session %p: Full topology %p from media session doesn't match current topology %p", this, FullTopology.Get(), CurrentTopology.Get());
			}
		}
		else
		{
			UE_LOGF(LogWmfMedia, Verbose, "Session %p: Failed to get full topology from media session", this, *WmfMedia::ResultToString(Result));
		}
	}
#endif

	// initialize new topology
	UpdateCharacteristics();

	// new media opened successfully
	if (SessionState == EMediaState::Preparing)
	{
		UE_LOGF(LogWmfMedia, Verbose, "Session %p: Topology %p ready", this, CurrentTopology.Get());

		SessionState = EMediaState::Stopped;
		DeferredEvents.Enqueue(EMediaEvent::MediaOpened);
	}
	else if (SessionState == EMediaState::Paused)
	{
		// Note: when paused, the new topology won't apply until the next session start,
		// so we request a scrub to the current time in order to update the video frame.
		JamWorkItem(FWorkItem([this](TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState) -> EWorkItemResult
			{
				auto State = static_cast<FTimeWorkItemState*>(BaseState);

				UE_LOGF(LogWmfMedia, Verbose, "Session %p: 'Re-Scrubbing' after topology change [SState=%d X=%d E=%d]", this, int(State->State), EventType == MESessionStarted, int(EventType));

				if (State->State == FTimeWorkItemState::EState::WaitDone)
				{
					return (EventType == MESessionStarted) ? EWorkItemResult::Done : EWorkItemResult::RetryLater;
				}

				if (CommitTime(WmfMediaSession::RequestedTimeCurrent, false))
				{
					State->State = FTimeWorkItemState::EState::WaitDone;
					return EWorkItemResult::RetryLater;
				}

				return EWorkItemResult::Done;
			}, MakeShared<FTimeWorkItemState, ESPMode::ThreadSafe>()));
	}
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
