// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Containers/Queue.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "IMediaControls.h"
#include "IMediaEventSink.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"

#include "Windows/AllowWindowsPlatformTypes.h"

class FWmfMediaTracks;
enum class EMediaEvent;


/**
 * Implements a media session that for handling asynchronous commands and callbacks.
 *
 * Many of the media playback features are asynchronous and do not take place
 * immediately, such as seeking and playback rate changes. A media session may
 * generate events during playback that are then handled by this class.
 *
 * Windows Media Foundation has a number of odd quirks and problems that require
 * special handling, such as certain state changes not being allowed, and some
 * calls causing occasional deadlocks. The added complexity in the implementation
 * of this class is for working around those issues.
 */
class FWmfMediaSession
	: public IMFAsyncCallback
	, public IMediaControls
{
public:

	/** Default constructor. */
	FWmfMediaSession();

public:

	/**
	 * Gets the session capabilities.
	 *
	 * @return Capabilities bit mask.
	 * @see GetEvents
	 */
	DWORD GetCapabilities() const
	{
		return Capabilities;
	}

	/**
	 * Gets all deferred player events.
	 *
	 * @param OutEvents Will contain the events.
	 * @see GetCapabilities
	 */
	void GetEvents(TArray<EMediaEvent>& OutEvents);

	/**
	 * Set which tracks object is being used by the player.
	 *
	 * @param InTracks Tracks object.
	 */
	void SetTracks(TSharedPtr<FWmfMediaTracks, ESPMode::ThreadSafe> InTracks);

	/**
	 * Initialize the media session.
	 *
	 * @param LowLatency Whether to enable low latency processing.
	 * @see SetTopolgy, Shutdown
	 */
	bool Initialize(bool LowLatency);

	/**
	 * Set the playback topology to be used by this session.
	 *
	 * @param InTopology The topology to set.
	 * @param InDuration Total duration of the media being played.
	 * @return true on success, false otherwise.
	 * @see Initialize, Shutdown
	 */
	bool SetTopology(const TComPtr<IMFTopology>& InTopology, FTimespan InDuration);

	/**
	 * Close the media session.
	 *
	 * @see Initialize
	 */
	void Shutdown();

public:

	//~ IMediaControls interface

	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override
	{ check(!"You have to call the override with additional options!"); return false; }
	virtual bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;

public:

	//~ IMFAsyncCallback interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP GetParameters(unsigned long* Flags, unsigned long* Queue);
	STDMETHODIMP Invoke(IMFAsyncResult* AsyncResult);
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

public:
	void Flush();

	void RequestMoreVideoData();

	enum class EWorkItemResult
	{
		Done,
		RetryLater
	};

	struct FBaseWorkItemState
	{
		virtual ~FBaseWorkItemState() = default;
	};

	class FWorkItem
	{
	public:
		typedef TFunction<EWorkItemResult(TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus, FBaseWorkItemState* BaseState)>	FWorkLoadType;

		FWorkItem() = default;
		FWorkItem(FWorkLoadType&& InWorkLoad, TSharedPtr<FBaseWorkItemState, ESPMode::ThreadSafe> InState = nullptr) : WorkLoad(MoveTemp(InWorkLoad)), State(InState) {}

		FWorkLoadType WorkLoad;
		TSharedPtr<FBaseWorkItemState, ESPMode::ThreadSafe> State;

		bool IsValid() const
		{
			return (bool)WorkLoad;
		}

		void Reset()
		{
			WorkLoad.Reset();
			State.Reset();
		}
	};

public:
	void EnqueueWorkItem(FWorkItem&& WorkItem);
	void JamWorkItem(FWorkItem&& WorkItem);

protected:
	struct FTopologyWorkItemState : public FBaseWorkItemState
	{
		enum class EState
		{
			Begin,
			Ready,
			ReadyNeedsRestart,
			WaitRestart,
		};

		EState State = EState::Begin;
	};

	struct FRateWorkItemState : public FBaseWorkItemState
	{
		enum class EState
		{
			Begin,
			Ready,
			ReadyNeedsRestart,
			WaitForStopThenSetAndRestart,
			WaitForPauseThenSetAndRestart,
			WaitForSet,
			WaitForSetAndRestart,
			WaitForRestart,
			WaitForPause,
			WaitForStart
		};

		EState State = EState::Begin;
		FTimespan LastTime;
	};

	struct FTimeWorkItemState : public FBaseWorkItemState
	{
		enum class EState
		{
			Begin,
			WaitDone,
		};

		EState State = EState::Begin;
	};

	/**
	 * Commit the specified play rate.
	 *
	 * The caller holds the lock to the critical section.
	 *
	 * @param Rate The play rate to commit.
	 * @see CommitTime, CommitTopology
	 */
	EWorkItemResult CommitRate(float Rate, FRateWorkItemState* State, MediaEventType EventType);

	/**
	 * Commit the specified play position.
	 *
	 * The caller holds the lock to the critical section.
	 *
	 * @param Time The play position to commit.
	 * @see CommitRate, CommitTopology
	 */
	bool CommitTime(FTimespan Time, bool bIsSeek);

	/**
	 * Commit the given playback topology.
	 *
	 * @param Topology The topology to set.
	 * @see CommitRate, CommitTime
	 */
	EWorkItemResult CommitTopology(IMFTopology* Topology, FTopologyWorkItemState* State, MediaEventType EventType);

	/** Get the latest characteristics from the current media source. */
	void UpdateCharacteristics();

	void ExecuteNextWorkItems(TComPtr<IMFMediaEvent> MediaEvent, MediaEventType EventType, HRESULT EventStatus);
	void ResetWorkItemQueue();

	void SetSessionRate(float Rate);

private:

	/** Callback for the MEError event. */
	void HandleError(HRESULT EventStatus);

	/** Callback for the MESessionEnded event. */
	void HandleSessionEnded();

	void HandlePresentationEnded();

	/** Callback for the MESessionPaused event. */
	void HandleSessionPaused(HRESULT EventStatus);

	/** Callback for the MESessionRateChanged event. */
	void HandleSessionRateChanged(HRESULT EventStatus, IMFMediaEvent& Event);

	/** Callback for the MESessionStarted event. */
	void HandleSessionStarted(HRESULT EventStatus);

	/** Callback for the MESessionStopped event. */
	void HandleSessionStopped(HRESULT EventStatus);

	/** Callback for the MESessionTopologySet event. */
	void HandleSessionTopologySet(HRESULT EventStatus, IMFMediaEvent& Event);

	/** Callback for the MESessionTopologyStatus event. */
	void HandleSessionTopologyStatus(HRESULT EventStatus, IMFMediaEvent& Event);

private:

	/** Hidden destructor (this class is reference counted). */
	virtual ~FWmfMediaSession();

private:

	/** Whether the media session supports scrubbing. */
	bool CanScrub;

	/** The session's capabilities. */
	DWORD Capabilities;

	/** Synchronizes write access to session state. */
	mutable FCriticalSection CriticalSection;

	/** The duration of the media. */
	FTimespan CurrentDuration;

	/** The full playback topology currently set on the media session. */
	TComPtr<IMFTopology> CurrentTopology;

	/** Media events to be forwarded to main thread. */
	TQueue<EMediaEvent> DeferredEvents;

	/** The media session that handles all playback. */
	TComPtr<IMFMediaSession> MediaSession;

	/** MediaSession close event. */
	FEvent* MediaSessionCloseEvent;

	/** The last play head position before playback was stopped. */
	FTimespan LastTime;

	/** The media session's clock. */
	TComPtr<IMFPresentationClock> PresentationClock;

	/** Optional interface for controlling playback rates. */
	TComPtr<IMFRateControl> RateControl;

	/** Optional interface for querying supported playback rates. */
	TComPtr<IMFRateSupport> RateSupport;

	/** Holds a reference counter for this instance. */
	int32 RefCount;

	/** Deferred play rate change value. */
	TOptional<float> RequestedRate;

	/** Deferred playback topology to set. */
	TComPtr<IMFTopology> RequestedTopology;

	/** If true then RequestedTime is due to the video looping. */
	bool bIsRequestedTimeLoop;

	/** If true then RequestedTime is due to the video seeking. */
	bool bIsRequestedTimeSeek;

	float LastSetRate;

	/** The session's internal playback rate (not necessarily the same as GetRate). */
	float SessionRate;

	/** The session's last non-zero internal playback rate. */
	float UnpausedSessionRate;

	/** The session's current state (not necessarily the same as GetState). */
	EMediaState SessionState;

	/** Whether playback should loop to the beginning. */
	bool ShouldLoop;

	/** Current status flags. */
	EMediaStatus Status;

	/** Flag to signal shutdown in progress */
	bool bShuttingDown;

	/** The thinned play rates that the current media session supports. */
	TRangeSet<float> ThinnedRates;

	/** The unthinned play rates that the current media session supports. */
	TRangeSet<float> UnthinnedRates;

	/** Pointer to the tracks from the player. */
	TWeakPtr<FWmfMediaTracks, ESPMode::ThreadSafe> Tracks;

	/** True if we are waiting for the tracks to send out its last samples before we "end". */
	bool bIsWaitingForEnd;

	/** Next requested sequence index on a Seek() request. */
	TOptional<int32> NextSequenceIndex;

	/** True if seek is logically in progress */
	bool bSeekActive;

	/** Queue of work jobs for the player */
	TArray<FWorkItem> WorkItemQueue;

	/** The work item currently in progress (if valid) */
	FWorkItem CurrentWorkItem;

	/** Number of seek wokr-items in the queue currently */
	int32 NumSeeksInWorkQueue;

};


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
