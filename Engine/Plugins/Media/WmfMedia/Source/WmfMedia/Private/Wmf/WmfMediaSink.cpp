// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaSink.h"


#if WMFMEDIA_SUPPORTED_PLATFORM

#include "DXGI.h"

#include "Misc/CString.h"
#include "Misc/ScopeLock.h"

#include "RHI.h"

#include "WmfMediaStreamSink.h"
#include "WmfMediaUtils.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <d3d11.h>
#include <d3d12.h>
#include <dxgi1_4.h>
#include "Windows/HideWindowsPlatformTypes.h"

#include "ID3D11DynamicRHI.h"

/* FWmfMediaStreamSink structors
 *****************************************************************************/

FWmfMediaSink::FWmfMediaSink()
	: RefCount(0)
	, DXGIManager(nullptr)
	, D3D11Device(nullptr)
	, D3DImmediateContext(nullptr)
{ }


FWmfMediaSink::~FWmfMediaSink()
{
	StreamSink.Reset();
	check(RefCount == 0);
}


/* FWmfMediaStreamSink interface
 *****************************************************************************/

bool FWmfMediaSink::Initialize(TComPtr<FWmfMediaStreamSink> InStreamSink)
{
	FScopeLock Lock(&CriticalSection);

	if (GDynamicRHI == nullptr)
	{
		return false;
	}

	if (!InStreamSink->Initialize(*this))
	{
		return false;
	}

	StreamSink = InStreamSink;

	return CreateDXGIManagerAndDevice();
}


/* IMFClockStateSink interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaSink::OnClockPause(MFTIME hnsSystemTime)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return StreamSink->Pause();
}


STDMETHODIMP FWmfMediaSink::OnClockRestart(MFTIME hnsSystemTime)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return StreamSink->Restart();
}


STDMETHODIMP FWmfMediaSink::OnClockSetRate(MFTIME hnsSystemTime, float flRate)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	StreamSink->SetClockRate(flRate);

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::OnClockStart(MFTIME hnsSystemTime, LONGLONG llClockStartOffset)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return StreamSink->Start();
}


STDMETHODIMP FWmfMediaSink::OnClockStop(MFTIME hnsSystemTime)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return StreamSink->Stop();
}


/* IMFGetService interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaSink::GetService(__RPC__in REFGUID guidService, __RPC__in REFIID riid, __RPC__deref_out_opt LPVOID* ppvObject)
{
	if (guidService == MF_RATE_CONTROL_SERVICE)
	{
		return QueryInterface(riid, ppvObject);
	}
	
	if (guidService == MR_VIDEO_ACCELERATION_SERVICE)
	{
		HRESULT Result = S_OK;
		if (riid == __uuidof(IMFDXGIDeviceManager))
		{
			if (nullptr != DXGIManager)
			{
				*ppvObject = (void*) static_cast<IUnknown*>(DXGIManager);
				((IUnknown*)*ppvObject)->AddRef();
			}
			else
			{
				Result = E_NOINTERFACE;
			}
		}
		else
		{
			Result = E_NOINTERFACE;
		}

		return Result;
	}

	return MF_E_UNSUPPORTED_SERVICE;
}


/* IMFMediaSink interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaSink::AddStreamSink(DWORD dwStreamSinkIdentifier, __RPC__in_opt IMFMediaType* pMediaType, __RPC__deref_out_opt IMFStreamSink** ppStreamSink)
{
	return MF_E_STREAMSINKS_FIXED;
}


STDMETHODIMP FWmfMediaSink::GetCharacteristics(__RPC__out DWORD* pdwCharacteristics)
{
	if (pdwCharacteristics == NULL)
	{
		return E_POINTER;
	}

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pdwCharacteristics = MEDIASINK_FIXED_STREAMS | MEDIASINK_RATELESS;

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::GetPresentationClock(__RPC__deref_out_opt IMFPresentationClock** ppPresentationClock)
{
	if (ppPresentationClock == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	if (!PresentationClock.IsValid())
	{
		return MF_E_NO_CLOCK; // presentation clock not set yet
	}

	*ppPresentationClock = PresentationClock;
	(*ppPresentationClock)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::GetStreamSinkById(DWORD dwIdentifier, __RPC__deref_out_opt IMFStreamSink** ppStreamSink)
{
	if (ppStreamSink == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	DWORD Identifier = 0;

	if (FAILED(StreamSink->GetIdentifier(&Identifier)) || (Identifier != dwIdentifier))
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	*ppStreamSink = StreamSink;
	(*ppStreamSink)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::GetStreamSinkByIndex(DWORD dwIndex, __RPC__deref_out_opt IMFStreamSink** ppStreamSink)
{
	if (ppStreamSink == NULL)
	{
		return E_POINTER;
	}

	if (dwIndex > 0)
	{
		return MF_E_INVALIDINDEX; // stream count is fixed at 1
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*ppStreamSink = StreamSink;
	(*ppStreamSink)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::GetStreamSinkCount(__RPC__out DWORD* pcStreamSinkCount)
{
	if (pcStreamSinkCount == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pcStreamSinkCount = 1; // stream count is fixed at 1

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::RemoveStreamSink(DWORD dwStreamSinkIdentifier)
{
	return MF_E_STREAMSINKS_FIXED;
}


STDMETHODIMP FWmfMediaSink::SetPresentationClock(__RPC__in_opt IMFPresentationClock* pPresentationClock)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	// remove ourselves from old clock
	if (PresentationClock.IsValid())
	{
		const HRESULT Result = PresentationClock->RemoveClockStateSink(this);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Failed to remove media sink from presentation clock: %ls", *WmfMedia::ResultToString(Result));
			return Result;
		}
	}

	// Register ourselves to get state notifications from the new clock.
	if (pPresentationClock != NULL)
	{
		TComPtr<IMFRateControl> RateControl;

		HRESULT Result = pPresentationClock->QueryInterface(IID_PPV_ARGS(&RateControl));
 		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Failed to retrieve rate interface from presentation clock: %ls", *WmfMedia::ResultToString(Result));
			return Result;
		}

		float Rate;
		Result = RateControl->GetRate(NULL, &Rate);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Failed to retrieve rate from presentation clock: %ls", *WmfMedia::ResultToString(Result));
			return Result;
		}

		// Make sure the sink knows the current clock rate (the presentation clock may only be known after initial rate setup is already done)
		StreamSink->SetClockRate(Rate);

		Result = pPresentationClock->AddClockStateSink(this);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Failed to add media sink to presentation clock: %ls", *WmfMedia::ResultToString(Result));
			return Result;
		}
	}
	else
	{
		StreamSink->SetClockRate(-1.0f);
	}

	PresentationClock = pPresentationClock;

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::Shutdown()
{
	FScopeLock Lock(&CriticalSection);

	if (StreamSink.IsValid())
	{
		StreamSink->Shutdown();
		StreamSink.Reset();
	}

	if (PresentationClock.IsValid())
	{
		PresentationClock->RemoveClockStateSink(this);
		PresentationClock.Reset();
	}

	return MF_E_SHUTDOWN;
}


/* IMFRateSupport interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaSink::GetFastestRate(MFRATE_DIRECTION eDirection, BOOL fThin, _Out_ float* pflRate)
{
	if (pflRate == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pflRate = (eDirection == MFRATE_FORWARD) ? FLT_MAX : FLT_MIN;

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::GetSlowestRate(MFRATE_DIRECTION eDirection, BOOL fThin, _Out_ float* pflRate)
{
	if (pflRate == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pflRate = 0.0;

	return S_OK;
}


STDMETHODIMP FWmfMediaSink::IsRateSupported(BOOL fThin, float flRate, __RPC__inout_opt float* pflNearestSupportedRate)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamSink.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return S_OK;
}


/* IUnknown interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FWmfMediaSink::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

STDMETHODIMP FWmfMediaSink::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FWmfMediaSink, IMFClockStateSink),
		QITABENT(FWmfMediaSink, IMFGetService),
		QITABENT(FWmfMediaSink, IMFMediaSink),
		QITABENT(FWmfMediaSink, IMFMediaSinkPreroll),
		QITABENT(FWmfMediaSink, IMFRateSupport),
		{ 0 }
	};

	return QISearch(this, QITab, RefID, Object);
}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif


STDMETHODIMP_(ULONG) FWmfMediaSink::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);

	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


bool FWmfMediaSink::CreateDXGIManagerAndDevice()
{
	HRESULT Result = S_OK;

	UINT resetToken = 0;

	if (nullptr == DXGIManager)
	{
		if (GDynamicRHI == nullptr)
		{
			UE_LOGF(LogWmfMedia, Error, "Invalid Dynamic RHI");
			return false;
		}

		TRefCountPtr<IDXGIAdapter> DXGIAdapter;
		ERHIInterfaceType RHIType = RHIGetInterfaceType();

		// Is this D3D11?
		if(RHIType == ERHIInterfaceType::D3D11)
		{
			Result = MFCreateDXGIDeviceManager(&resetToken, &DXGIManager);
			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Error, "Unable to Create DXGI Device Manager: %ls", *WmfMedia::ResultToString(Result));
				return false;
			}

			ID3D11Device* PreExistingD3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();

			TComPtr<IDXGIDevice> DXGIDevice;
			PreExistingD3D11Device->QueryInterface(__uuidof(IDXGIDevice), (void**)&DXGIDevice);

			DXGIDevice->GetAdapter((IDXGIAdapter**)DXGIAdapter.GetInitReference());
		}
		else
		{
			DXGIManager.Reset();
			D3D11Device.Reset();
			UE_LOGF(LogWmfMedia, Verbose, "Dynamic RHI is not D3D11, skipping DX11 device creation for AV decoding");
			return true;
		}

		// Create device from same adapter as already existing device
		D3D_FEATURE_LEVEL FeatureLevel;

		uint32 DeviceCreationFlags = 0;

		if (FParse::Param(FCommandLine::Get(), TEXT("d3ddebug")))
		{
			DeviceCreationFlags |= D3D11_CREATE_DEVICE_DEBUG;
		}


		Result = D3D11CreateDevice(
			DXGIAdapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			nullptr,
			DeviceCreationFlags,
			nullptr,
			0,
			D3D11_SDK_VERSION,
			&D3D11Device,
			&FeatureLevel,
			&D3DImmediateContext);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Unable to Create D3D11 Device: %ls", *WmfMedia::ResultToString(Result));
			return false;
		}

		if (FeatureLevel < D3D_FEATURE_LEVEL_9_3)
		{
			UE_LOGF(LogWmfMedia, Error, "Unable to Create D3D11 Device with feature level 9.3 or above");
			return false;
		}

		Result = DXGIManager->ResetDevice(D3D11Device, resetToken);
		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Error, "Unable to Reset D3D11 Device: %ls", *WmfMedia::ResultToString(Result));
			return false;
		}

		ID3D10Multithread* MultiThread = nullptr;

		D3D11Device->QueryInterface(__uuidof(ID3D10Multithread), (void**)&MultiThread);
		if (MultiThread)
		{
			MultiThread->SetMultithreadProtected(true);
			MultiThread->Release();
		}
		else
		{
			UE_LOGF(LogWmfMedia, Error, "Unable to activate multi-threading on device: %p", D3D11Device.Get());
			return false;
		}

		UE_LOGF(LogWmfMedia, Verbose, "D3D11 Device Created: %p", D3D11Device.Get());
		return true;
	}
	else
	{
		return true;
	}
}

#endif

