// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "HAL/CriticalSection.h"

class FWmfMediaStreamSink;
struct ID3D11DeviceContext;
struct ID3D11Device;


/**
 * Implements a media sink object for the WMF pipeline.
 */
class FWmfMediaSink
	: public IMFClockStateSink
	, public IMFGetService
	, public IMFMediaSink
	, public IMFRateSupport
{
public:

	/** Default constructor. */
	FWmfMediaSink();

public:

	/**
	 * Initialize this sink.
	 *
	 * @param InStreamSink The stream sink to use for the fixed stream.
	 * @return true on success, false otherwise.
	 */
	bool Initialize(TComPtr<FWmfMediaStreamSink> InStreamSink);

public:

	//~ IMFClockStateSink interface

	STDMETHODIMP OnClockPause(MFTIME hnsSystemTime);
	STDMETHODIMP OnClockRestart(MFTIME hnsSystemTime);
	STDMETHODIMP OnClockSetRate(MFTIME hnsSystemTime, float flRate);
	STDMETHODIMP OnClockStart(MFTIME hnsSystemTime, LONGLONG llClockStartOffset);
	STDMETHODIMP OnClockStop(MFTIME hnsSystemTime);

public:

	//~ IMFGetService interface

	STDMETHODIMP GetService(__RPC__in REFGUID guidService, __RPC__in REFIID riid, __RPC__deref_out_opt LPVOID* ppvObject);

public:

	// ~IMFMediaSink interface

	STDMETHODIMP AddStreamSink(DWORD dwStreamSinkIdentifier, __RPC__in_opt IMFMediaType* pMediaType, __RPC__deref_out_opt IMFStreamSink** ppStreamSink);
	STDMETHODIMP GetCharacteristics(__RPC__out DWORD* pdwCharacteristics);
	STDMETHODIMP GetPresentationClock(__RPC__deref_out_opt IMFPresentationClock** ppPresentationClock);
	STDMETHODIMP GetStreamSinkById(DWORD dwIdentifier, __RPC__deref_out_opt IMFStreamSink** ppStreamSink);
	STDMETHODIMP GetStreamSinkByIndex(DWORD dwIndex, __RPC__deref_out_opt IMFStreamSink** ppStreamSink);
	STDMETHODIMP GetStreamSinkCount(__RPC__out DWORD* pcStreamSinkCount);
	STDMETHODIMP RemoveStreamSink(DWORD dwStreamSinkIdentifier);
	STDMETHODIMP SetPresentationClock(__RPC__in_opt IMFPresentationClock* pPresentationClock);
	STDMETHODIMP Shutdown();

public:

	//~ IMFRateSupport interface

	STDMETHODIMP GetFastestRate(MFRATE_DIRECTION eDirection, BOOL fThin, _Out_ float* pflRate);
	STDMETHODIMP GetSlowestRate(MFRATE_DIRECTION eDirection, BOOL fThin, _Out_ float* pflRate);
	STDMETHODIMP IsRateSupported(BOOL fThin, float flRate, __RPC__inout_opt float* pflNearestSupportedRate);

public:

	//~ IUnknown interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

	ID3D11DeviceContext* GetImmediateContext() const { return D3DImmediateContext; }
	ID3D11Device* GetDevice() const { return D3D11Device; }

private:

	/** Hidden destructor (this class is reference counted). */
	virtual ~FWmfMediaSink();

	bool CreateDXGIManagerAndDevice();

private:

	/** Critical section for synchronizing access to this sink. */
	FCriticalSection CriticalSection;

	/** Holds a reference counter for this instance. */
	int32 RefCount;

	/** The presentation clock used by this sink. */
	TComPtr<IMFPresentationClock> PresentationClock;

	/** The stream sink. */
	TComPtr<FWmfMediaStreamSink> StreamSink;

	/** Support for Hardware Accelerated **/
	TComPtr<IMFDXGIDeviceManager> DXGIManager;
	TComPtr<ID3D11Device> D3D11Device;
	TComPtr<ID3D11DeviceContext> D3DImmediateContext;
};

#endif
