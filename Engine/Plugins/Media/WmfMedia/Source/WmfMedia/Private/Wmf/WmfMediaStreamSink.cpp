// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaStreamSink.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Math/UnrealMathUtility.h"
#include "MediaSampleQueueDepths.h"
#include "Misc/ScopeLock.h"
#include "RenderingThread.h"
#include "timeapi.h"
#include "WmfMediaCodec/WmfMediaDecoder.h"
#include "WmfMediaHardwareVideoDecodingTextureSample.h"
#include "WmfMediaSink.h"
#include "WmfMediaUtils.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <d3d11.h>

static constexpr uint32 kMaxFramesInOutputQueue = 6;

/* FWmfMediaStreamSink static functions
 *****************************************************************************/

bool FWmfMediaStreamSink::Create(const GUID& MajorType, TComPtr<FWmfMediaStreamSink>& OutSink)
{
	TComPtr<FWmfMediaStreamSink> StreamSink = new FWmfMediaStreamSink(MajorType, 1);
	TComPtr<FWmfMediaSink> MediaSink = new FWmfMediaSink();

	if (!MediaSink->Initialize(StreamSink))
	{
		return false;
	}

	OutSink = StreamSink;

	return true;
}


/* FWmfMediaStreamSink structors
 *****************************************************************************/

FWmfMediaStreamSink::FWmfMediaStreamSink(const GUID& InMajorType, DWORD InStreamId)
	: Owner(nullptr)
	, Decoder(nullptr)
	, RefCount(0)
	, StreamId(InStreamId)
	, StreamType(InMajorType)
	, ClockRate(-1.0f)
	, VideoSamplePool(nullptr)
	, VideoSampleQueue(nullptr)
	, NumRequestsInFlight(0)
	, NumRequestsInFlightWithSystem(0)
	, bSeekInProgress(false)
	, bScrubbingInProgress(false)
	, bSystemIsScrubbing(false)
	, bSegmentEnded(true)
	, bShowSubTypeErrorMessage(true)
{
	UE_LOGF(LogWmfMedia, Verbose, "StreamSink %p: Created with stream type %ls", this, *WmfMedia::MajorTypeToString(StreamType));
}


FWmfMediaStreamSink::~FWmfMediaStreamSink()
{
	check(RefCount == 0);

	UE_LOGF(LogWmfMedia, Verbose, "StreamSink %p: Destroyed", this);
}


/* FWmfMediaStreamSink interface
 *****************************************************************************/

bool FWmfMediaStreamSink::Initialize(FWmfMediaSink& InOwner)
{
	FScopeLock Lock(&CriticalSection);

	const HRESULT Result = ::MFCreateEventQueue(&EventQueue);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, Verbose, "StreamSink %p: Failed to create event queue for stream sink: %ls", this, *WmfMedia::ResultToString(Result));
		return false;
	}

	Owner = &InOwner;

	return true;
}

void FWmfMediaStreamSink::SetDecoder(WmfMediaDecoder* InDecoder)
{
	Decoder = InDecoder;
}


HRESULT FWmfMediaStreamSink::Pause()
{
	FScopeLock Lock(&CriticalSection);
	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink::Pause");
	return QueueEvent(MEStreamSinkPaused, GUID_NULL, S_OK, NULL);
}


HRESULT FWmfMediaStreamSink::Restart()
{
	HRESULT Result = QueueEvent(MEStreamSinkStarted, GUID_NULL, S_OK, NULL);

	if (FAILED(Result))
	{
		return Result;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p:Restart Request Sample", this);

	return S_OK;
}


void FWmfMediaStreamSink::Shutdown()
{
	FScopeLock Lock(&CriticalSection);

	if (EventQueue.IsValid())
	{
		EventQueue->Shutdown();
		EventQueue.Reset();
	}

	CurrentMediaType.Reset();
}


HRESULT FWmfMediaStreamSink::Start()
{
	FScopeLock Lock(&CriticalSection);

	bScrubbingInProgress = (ClockRate == 0.0f);
	bSystemIsScrubbing = bScrubbingInProgress;

	HRESULT Result = QueueEvent(MEStreamSinkStarted, GUID_NULL, S_OK, NULL);

	if (FAILED(Result))
	{
		return Result;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p:Start Request Sample", this);

	bSegmentEnded = false;
	bSeekInProgress = false;

	return S_OK;
}


HRESULT FWmfMediaStreamSink::Stop()
{
	HRESULT Res;
	TFunction<void()> TempAction;
	{
	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink::Stop");

	FScopeLock Lock(&CriticalSection);

	NumRequestsInFlight -= NumRequestsInFlightWithSystem;
	NumRequestsInFlightWithSystem = 0;

	Res = QueueEvent(MEStreamSinkStopped, GUID_NULL, S_OK, NULL);

	if ((NumRequestsInFlight == 0) && OnIdleAction)
	{
		TempAction = MoveTemp(OnIdleAction);
	}
	}

	if (TempAction)
	{
		TempAction();
	}

	return Res;
}

/* IMFGetService interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaStreamSink::GetService(__RPC__in REFGUID guidService, __RPC__in REFIID riid, __RPC__deref_out_opt LPVOID* ppvObject)
{
	return Owner->GetService(guidService, riid, ppvObject);
}


/* IMFMediaEventGenerator interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaStreamSink::BeginGetEvent(IMFAsyncCallback* pCallback, IUnknown* pState)
{
	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return EventQueue->BeginGetEvent(pCallback, pState);
}


STDMETHODIMP FWmfMediaStreamSink::EndGetEvent(IMFAsyncResult* pResult, IMFMediaEvent** ppEvent)
{
	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	HRESULT Result = EventQueue->EndGetEvent(pResult, ppEvent);

	if (Result == S_OK)
	{
		MediaEventType Type;
		(*ppEvent)->GetType(&Type);
		if (Type == MEStreamSinkRequestSample)
		{
			++NumRequestsInFlightWithSystem;
		}
	}

	return Result;
}


STDMETHODIMP FWmfMediaStreamSink::GetEvent(DWORD dwFlags, IMFMediaEvent** ppEvent)
{
	TComPtr<IMFMediaEventQueue> TempQueue;
	{
		FScopeLock Lock(&CriticalSection);

		if (!EventQueue.IsValid())
		{
			return MF_E_SHUTDOWN;
		}

		TempQueue = EventQueue;
	}

	HRESULT Result = TempQueue->GetEvent(dwFlags, ppEvent);

	if (Result == S_OK)
	{
		MediaEventType Type;
		(*ppEvent)->GetType(&Type);
		if (Type == MEStreamSinkRequestSample)
		{
			FScopeLock Lock(&CriticalSection);
			++NumRequestsInFlightWithSystem;
		}
	}

	return Result;
}


STDMETHODIMP FWmfMediaStreamSink::QueueEvent(MediaEventType met, REFGUID extendedType, HRESULT hrStatus, const PROPVARIANT* pvValue)
{
	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return EventQueue->QueueEventParamVar(met, extendedType, hrStatus, pvValue);
}


/* IMFMediaTypeHandler interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaStreamSink::GetCurrentMediaType(_Outptr_ IMFMediaType** ppMediaType)
{
	FScopeLock Lock(&CriticalSection);

	if (ppMediaType == NULL)
	{
		return E_POINTER;
	}

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	if (!CurrentMediaType.IsValid())
	{
		return MF_E_NOT_INITIALIZED;
	}

	*ppMediaType = CurrentMediaType;
	(*ppMediaType)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::GetMajorType(__RPC__out GUID* pguidMajorType)
{
	if (pguidMajorType == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	if (!CurrentMediaType.IsValid())
	{
		return MF_E_NOT_INITIALIZED;
	}

	return CurrentMediaType->GetGUID(MF_MT_MAJOR_TYPE, pguidMajorType);
}


STDMETHODIMP FWmfMediaStreamSink::GetMediaTypeByIndex(DWORD dwIndex, _Outptr_ IMFMediaType** ppType)
{
	if (ppType == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	// get supported media type
	TArray<TComPtr<IMFMediaType>> SupportedTypes = WmfMedia::GetSupportedMediaTypes(StreamType);

	if (!SupportedTypes.IsValidIndex(dwIndex))
	{
		return MF_E_NO_MORE_TYPES;
	}

	TComPtr<IMFMediaType> SupportedType = SupportedTypes[dwIndex];

	if (!SupportedType.IsValid())
	{
		return MF_E_INVALIDMEDIATYPE;
	}

	// create result type
	TComPtr<IMFMediaType> MediaType;
	{
		HRESULT Result = ::MFCreateMediaType(&MediaType);

		if (FAILED(Result))
		{
			return Result;
		}

		Result = SupportedType->CopyAllItems(MediaType);

		if (FAILED(Result))
		{
			return Result;
		}
	}

	*ppType = MediaType;
	(*ppType)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::GetMediaTypeCount(__RPC__out DWORD* pdwTypeCount)
{
	if (pdwTypeCount == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pdwTypeCount = (DWORD)WmfMedia::GetSupportedMediaTypes(StreamType).Num();

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::IsMediaTypeSupported(IMFMediaType* pMediaType, _Outptr_opt_result_maybenull_ IMFMediaType** ppMediaType)
{
	if (ppMediaType != NULL)
	{
		*ppMediaType = NULL;
	}

	if (pMediaType == NULL)
	{
		return E_POINTER;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Checking if media type is supported:\n%ls", this, *WmfMedia::DumpAttributes(*pMediaType));

	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	// get requested major type
	GUID MajorType;
	{
		const HRESULT Result = pMediaType->GetGUID(MF_MT_MAJOR_TYPE, &MajorType);

		if (FAILED(Result))
		{
			return Result;
		}
	}

	if (MajorType != StreamType)
	{
		UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Media type doesn't match stream type %ls", this, *WmfMedia::MajorTypeToString(StreamType));
		return MF_E_INVALIDMEDIATYPE;
	}

	// compare media 
	const DWORD CompareFlagsData = MF_MEDIATYPE_EQUAL_MAJOR_TYPES | MF_MEDIATYPE_EQUAL_FORMAT_TYPES | MF_MEDIATYPE_EQUAL_FORMAT_DATA;
	const DWORD CompareFlagsUserData = MF_MEDIATYPE_EQUAL_MAJOR_TYPES | MF_MEDIATYPE_EQUAL_FORMAT_TYPES | MF_MEDIATYPE_EQUAL_FORMAT_USER_DATA;

	for (const TComPtr<IMFMediaType>& MediaType : WmfMedia::GetSupportedMediaTypes(StreamType))
	{
		if (!MediaType.IsValid())
		{
			continue;
		}

		DWORD OutFlags = 0;
		const HRESULT Result = MediaType->IsEqual(pMediaType, &OutFlags);

		if (SUCCEEDED(Result) && (((OutFlags & CompareFlagsData) == CompareFlagsData) || ((OutFlags & CompareFlagsUserData) == CompareFlagsUserData)))
		{
			UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Media type is supported", this);
			return S_OK;
		}
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Media type is not supported", this);

	return MF_E_INVALIDMEDIATYPE;
}


STDMETHODIMP FWmfMediaStreamSink::SetCurrentMediaType(IMFMediaType* pMediaType)
{
	if (pMediaType == NULL)
	{
		return E_POINTER;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Setting current media type:\n%ls", this, *WmfMedia::DumpAttributes(*pMediaType));

	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	const HRESULT Result = IsMediaTypeSupported(pMediaType, NULL);

	if (FAILED(Result))
	{
		UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Tried to set unsupported media type", this);
		return Result;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Current media type set", this);

	CurrentMediaType = pMediaType;

	return S_OK;
}


/* IMFStreamSink interface
 *****************************************************************************/

STDMETHODIMP FWmfMediaStreamSink::Flush()
{
	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	VideoSampleQueue->RequestFlush();

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::GetIdentifier(__RPC__out DWORD* pdwIdentifier)
{
	if (pdwIdentifier == NULL)
	{
		return E_POINTER;
	}

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*pdwIdentifier = StreamId;

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::GetMediaSink(__RPC__deref_out_opt IMFMediaSink** ppMediaSink)
{
	if (ppMediaSink == NULL)
	{
		return E_POINTER;
	}

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	*ppMediaSink = Owner;
	(*ppMediaSink)->AddRef();

	return S_OK;
}


STDMETHODIMP FWmfMediaStreamSink::GetMediaTypeHandler(__RPC__deref_out_opt IMFMediaTypeHandler** ppHandler)
{
	if (ppHandler == NULL)
	{
		return E_POINTER;
	}

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	return QueryInterface(IID_IMFMediaTypeHandler, (void**)ppHandler);
}


STDMETHODIMP FWmfMediaStreamSink::PlaceMarker(MFSTREAMSINK_MARKER_TYPE eMarkerType, __RPC__in const PROPVARIANT* pvarMarkerValue, __RPC__in const PROPVARIANT* pvarContextValue)
{
	FScopeLock Lock(&CriticalSection);

	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	if (eMarkerType == MFSTREAMSINK_MARKER_ENDOFSEGMENT)
	{
		bSegmentEnded = true;
	}

	// Pass on all markers via event...
	PROPVARIANT* MarkerContext = new PROPVARIANT;

	if (pvarContextValue != NULL)
	{
		HRESULT Result = ::PropVariantCopy(MarkerContext, pvarContextValue);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Verbose, "StreamSink %p: Failed to copy marker context: %ls", this, *WmfMedia::ResultToString(Result));
			delete MarkerContext;

			return Result;
		}
	}

	QueueEvent(MEStreamSinkMarker, GUID_NULL, S_OK, MarkerContext);
	PropVariantClear(MarkerContext);
	delete MarkerContext;

	return S_OK;
}


void FWmfMediaStreamSink::OnSeekStarted()
{
	bSeekInProgress = true;
}


void FWmfMediaStreamSink::OnSessionEnded()
{
	TFunction<void()> TempAction;
	{
	FScopeLock Lock(&CriticalSection);
	
	NumRequestsInFlight -= NumRequestsInFlightWithSystem;
	NumRequestsInFlightWithSystem = 0;

	if ((NumRequestsInFlight == 0) && OnIdleAction)
	{
		TempAction = MoveTemp(OnIdleAction);
	}
	}

	if (TempAction)
	{
		TempAction();
	}
}


void FWmfMediaStreamSink::OnRequestMoreData()
{
	FScopeLock Lock(&CriticalSection);

	if (!OnIdleAction)
	{
		/*
		* - We need the event queue
		* - We do not accept new request if we just received a "end of segment" & no new start since
		* - The output queue must not be full
		* - If scrubbing, we only accept one request once per "start with zero" phase
		*/
		if (EventQueue.IsValid() && !bSegmentEnded && !IsVideoSampleQueueFull() && (!bSystemIsScrubbing || (bScrubbingInProgress && NumRequestsInFlight == 0)))
		{
			QueueEvent(MEStreamSinkRequestSample, GUID_NULL, S_OK, NULL);
			++NumRequestsInFlight;
		}
	}
}


bool FWmfMediaStreamSink::ExecuteOnceNoPendingRequests(TFunction<void()>&& ExecuteOnIdle)
{
	FScopeLock Lock(&CriticalSection);

	if (NumRequestsInFlight == 0)
	{
		return false;
	}

	if (!OnIdleAction)
	{
		OnIdleAction = MoveTemp(ExecuteOnIdle);
	}
	return true;
}


STDMETHODIMP FWmfMediaStreamSink::ProcessSample(__RPC__in_opt IMFSample* pSample)
{
	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Process Sample", this);

	if (pSample == nullptr)
	{
		return E_POINTER;
	}

	TFunction<void()> TempAction;

	{
	FScopeLock Lock(&CriticalSection);
	
	if (!EventQueue.IsValid())
	{
		return MF_E_SHUTDOWN;
	}

	NumRequestsInFlight = FMath::Max(NumRequestsInFlight-1, 0);
	NumRequestsInFlightWithSystem = FMath::Max(NumRequestsInFlightWithSystem-1, 0);

	bool bIdleWaitInProgress = !!OnIdleAction;

	if (bIdleWaitInProgress && NumRequestsInFlight == 0)
	{
		TempAction = MoveTemp(OnIdleAction);
	}

	if (!CurrentMediaType.IsValid())
	{
		UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink %p: Stream received a sample while not having a valid media type set", this);
		return MF_E_INVALIDMEDIATYPE;
	}

	// get sample time
	LONGLONG Time = 0;
	{
		const HRESULT Result = pSample->GetSampleTime(&Time);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, VeryVerbose, "Failed to get time from sink sample: %ls", *WmfMedia::ResultToString(Result));
			return Result;
		}
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink::ProcessSample Sample time %f", FTimespan::FromMicroseconds(Time / 10).GetTotalSeconds());

	if (!bSeekInProgress)
	{
		if (bScrubbingInProgress)
		{
			bScrubbingInProgress = false;

			// Send out scrub complete event.
			QueueEvent(MEStreamSinkScrubSampleComplete, GUID_NULL, S_OK, NULL);
		}

		if (IsVideoSampleQueueFull())
		{
			// This should not happen, but if it does: throw an error (we'll loose the sample)
			return MF_E_INVALID_STATE_TRANSITION;
		}

		// Take sample and put it into the output queue
		CopyTextureAndEnqueueSample(pSample);

		if (!bIdleWaitInProgress && !IsVideoSampleQueueFull() && ClockRate != 0.0f)
		{
			UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink::ProcessSample Request Sample2");
			QueueEvent(MEStreamSinkRequestSample, GUID_NULL, S_OK, NULL);
			++NumRequestsInFlight;
		}
	}
	}

	if (TempAction)
	{
		TempAction();
	}

	return S_OK;
}


/* IUnknown interface
 *****************************************************************************/

STDMETHODIMP_(ULONG) FWmfMediaStreamSink::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


#if _MSC_VER == 1900
	#pragma warning(push)
	#pragma warning(disable:4838)
#endif

STDMETHODIMP FWmfMediaStreamSink::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FWmfMediaStreamSink, IMFGetService),
		QITABENT(FWmfMediaStreamSink, IMFMediaTypeHandler),
		QITABENT(FWmfMediaStreamSink, IMFStreamSink),
		{ 0 }
	};

	return QISearch(this, QITab, RefID, Object);
}

#if _MSC_VER == 1900
	#pragma warning(pop)
#endif


STDMETHODIMP_(ULONG) FWmfMediaStreamSink::Release()
{
	int32 CurrentRefCount = FPlatformAtomics::InterlockedDecrement(&RefCount);

	if (CurrentRefCount == 0)
	{
		delete this;
	}

	return CurrentRefCount;
}


bool FWmfMediaStreamSink::IsVideoSampleQueueFull() const
{
	const int32 NumberOfQueueFrames = kMaxFramesInOutputQueue;
	const int32 MinNumberOfQueueFrames = FMath::Min(NumberOfQueueFrames, FMediaPlayerQueueDepths::MaxVideoSinkDepth);
	return ((VideoSampleQueue->Num() + NumRequestsInFlight)	>= MinNumberOfQueueFrames);
}


void FWmfMediaStreamSink::CopyTextureAndEnqueueSample(IMFSample* pSample)
{
	// CopyTextureAndEnqueueSample might get called after we have shutdown, so check first...
	if (!CurrentMediaType.IsValid())
	{
		return;
	}

	UE_LOGF(LogWmfMedia, VeryVerbose, "StreamSink::CopyTextureAndEnqueueSample Queue Size: %d", VideoSampleQueue->Num());

	if (!IsSampleStillDesired())
	{
		UE_LOGF(LogWmfMedia, VeryVerbose, "Sample is from previous sequence, dropping sample");
		return;
	}

	if (IsVideoSampleQueueFull())
	{
		UE_LOGF(LogWmfMedia, VeryVerbose, "Queue is full, dropping sample");
		return;
	}

	DWORD cBuffers = 0;
	TComPtr<IMFMediaBuffer> pBuffer;
	TComPtr<IMFDXGIBuffer> pDXGIBuffer;
	UINT dwViewIndex = 0;
	TComPtr<ID3D11Texture2D> pTexture2D;

	HRESULT Result = pSample->GetBufferCount(&cBuffers);
	if (FAILED(Result))
	{
		return;
	}
	if (1 == cBuffers)
	{
		Result = pSample->GetBufferByIndex(0, &pBuffer);
		if (FAILED(Result))
		{
			return;
		}
	}

	UINT32 DimX;
	UINT32 DimY;
	MFGetAttributeSize(CurrentMediaType, MF_MT_FRAME_SIZE, &DimX, &DimY);

	MFVideoArea videoArea = {};
	Result = CurrentMediaType->GetBlob(MF_MT_MINIMUM_DISPLAY_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
	if (FAILED(Result))
	{
		Result = CurrentMediaType->GetBlob(MF_MT_GEOMETRIC_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
		if (FAILED(Result))
		{
			videoArea.OffsetX.fract = 0;
			videoArea.OffsetX.value = 0;
			videoArea.OffsetY.fract = 0;
			videoArea.OffsetY.value = 0;
			videoArea.Area.cx = DimX;
			videoArea.Area.cy = DimY;
		}
	}


	LONGLONG SampleTime = 0;
	LONGLONG SampleDuration = 0;

	pSample->GetSampleTime(&SampleTime);
	pSample->GetSampleDuration(&SampleDuration);

	GUID Guid;
	if (SUCCEEDED(CurrentMediaType->GetGUID(MF_MT_SUBTYPE, &Guid)))
	{
		const TSharedRef<FWmfMediaHardwareVideoDecodingTextureSample, ESPMode::ThreadSafe> TextureSample = VideoSamplePool->AcquireShared();

		GUID WrappedFormatGuid;
		if (SUCCEEDED(CurrentMediaType->GetGUID(UE_WMF_PrivateFormatGUID, &WrappedFormatGuid)))
		{
			Guid = WrappedFormatGuid;
		}

		EPixelFormat PixelFormat = PF_Unknown;
		EPixelFormat AlphaPixelFormat = PF_Unknown;
		EMediaTextureSampleFormat MediaTextureSampleFormat = EMediaTextureSampleFormat::Undefined;
		bool bIsDestinationTextureSRGB = false;

		if (Guid == MFVideoFormat_NV12)
		{
			PixelFormat = PF_NV12;
			MediaTextureSampleFormat = EMediaTextureSampleFormat::CharNV12;
		}
		else if (Guid == MFVideoFormat_ARGB32)
		{
			PixelFormat = PF_B8G8R8A8;
			MediaTextureSampleFormat = EMediaTextureSampleFormat::CharBGRA;
		}
		else if (Guid == MFVideoFormat_L8)
		{
			// We misuse this for DXT1 as there is no GUID for it.
			MediaTextureSampleFormat = EMediaTextureSampleFormat::DXT1;
			PixelFormat = PF_DXT1;
			bIsDestinationTextureSRGB = true;
		}
		else if (Guid == MFVideoFormat_L16)
		{
			// We misuse this for DXT5 as there is no GUID for it.
			MediaTextureSampleFormat = EMediaTextureSampleFormat::DXT5;
			PixelFormat = PF_DXT5;
			bIsDestinationTextureSRGB = true;
		}
		else if (Guid == MFVideoFormat_RGB8)
		{
			// We misuse this for YCoCg_DXT5 as there is no GUID for it.
			MediaTextureSampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5;
			PixelFormat = PF_DXT5;
		}
		else if (Guid == MFVideoFormat_D16)
		{
			// We misuse this for YCoCg_DXT5_Alpha_BC4 as there is no GUID for it.
			MediaTextureSampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4;
			PixelFormat = PF_DXT5;
			AlphaPixelFormat = PF_BC4;
		}
		else // if (Guid == MFVideoFormat_Y416)
		{
			PixelFormat = PF_A16B16G16R16;
			MediaTextureSampleFormat = EMediaTextureSampleFormat::Y416;
		}

		// Are we using external buffers?
		if ((Decoder != nullptr) && (Decoder->IsExternalBufferEnabled()))
		{
			// Get buffer from the decoder.
			TArray<uint8> ExternalBuffer;
			if (Decoder->GetExternalBuffer(ExternalBuffer, SampleTime) == false)
			{
				UE_LOGF(LogWmfMedia, Error, "External buffer not found for time %f",
					FTimespan::FromMicroseconds(SampleTime / 10).GetTotalSeconds());
				return;
			}

			// Set pitch.
			LONG Pitch = 0;
			if (MediaTextureSampleFormat == EMediaTextureSampleFormat::DXT1)
			{
				Pitch = DimX * 2;
			}
			else if ((MediaTextureSampleFormat == EMediaTextureSampleFormat::DXT5) ||
				(MediaTextureSampleFormat == EMediaTextureSampleFormat::YCoCg_DXT5) ||
				(MediaTextureSampleFormat == EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4))
			{
				Pitch = DimX * 4;
			}
			else
			{
				Pitch = ExternalBuffer.Num() / DimY;
			}

			// Set up sample.
			TextureSample->InitializeExternal(&ExternalBuffer,
				FIntPoint(DimX, DimY), FIntPoint(DimX, DimY), MediaTextureSampleFormat,
				Pitch,
				AdjustTimeStamp(FTimespan::FromMicroseconds(SampleTime / 10), EMediaTrackType::Video),
				FTimespan::FromMicroseconds(SampleDuration / 10));

			TextureSample->SetPixelFormat(PixelFormat);
			TextureSample->SetIsDestinationTextureSRGB(bIsDestinationTextureSRGB);
			TextureSample->SetAlphaTexture(AlphaPixelFormat);
			VideoSampleQueue->Enqueue(TextureSample);
			UE_LOGF(LogWmfMedia, VeryVerbose, "Enqueued external buffer onto VideoSampleQueue.");
		}
		else
		{
			// Only with D3D11 we can pass on data as a texture...
			if (RHIGetInterfaceType() == ERHIInterfaceType::D3D11)
			{
				Result = pBuffer->QueryInterface(__uuidof(IMFDXGIBuffer), (LPVOID*)&pDXGIBuffer);
				if (FAILED(Result))
				{
					return;
				}

				Result = pDXGIBuffer->GetResource(__uuidof(ID3D11Texture2D), (LPVOID*)&pTexture2D);
				if (FAILED(Result))
				{
					return;
				}

				Result = pDXGIBuffer->GetSubresourceIndex(&dwViewIndex);
				if (FAILED(Result))
				{
					return;
				}

				ID3D11Texture2D* SharedTexture = TextureSample->InitializeSourceTexture(
					Owner->GetDevice(),
					AdjustTimeStamp(FTimespan::FromMicroseconds(SampleTime / 10), EMediaTrackType::Video),
					FTimespan::FromMicroseconds(SampleDuration / 10),
					FIntPoint(videoArea.Area.cx, videoArea.Area.cy),
					PixelFormat,
					MediaTextureSampleFormat);

				if (!SharedTexture)
				{
					return;
				}

				D3D11_BOX SrcBox;
				SrcBox.left = 0;
				SrcBox.top = 0;
				SrcBox.front = 0;
				SrcBox.right = videoArea.Area.cx;
				SrcBox.bottom = videoArea.Area.cy;
				SrcBox.back = 1;

				UE_LOGF(LogWmfMedia, VeryVerbose, "CopySubresourceRegion() ViewIndex:%d Time:%f", dwViewIndex, FTimespan::FromMicroseconds(SampleTime / 10).GetTotalSeconds());

				TComPtr<IDXGIKeyedMutex> KeyedMutex;
				SharedTexture->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);

				if (KeyedMutex)
				{
					// No wait on acquire since sample is new and key is 0.
					if (KeyedMutex->AcquireSync(0, 0) == S_OK)
					{
						Owner->GetImmediateContext()->CopySubresourceRegion(SharedTexture, 0, 0, 0, 0, pTexture2D, dwViewIndex, &SrcBox);

						// Mark texture as updated with key of 1
						// Sample will be read in FWmfMediaHardwareVideoDecodingParameters::ConvertTextureFormat_RenderThread
						KeyedMutex->ReleaseSync(1);
						VideoSampleQueue->Enqueue(TextureSample);
						UE_LOGF(LogWmfMedia, VeryVerbose, "Enqueued onto VideoSampleQueue.");
					}
				}
			}
			else
			{
				// Pass on sample data as CPU side buffer

				DWORD BufferSize = 0;
				if (pBuffer->GetCurrentLength(&BufferSize) != S_OK)
				{
					return;
				}

				uint8* Data = nullptr;
				if (pBuffer->Lock(&Data, NULL, NULL) == S_OK)
				{
					// MFW expects the sample dimension for NV12/P010 to be scaled to include "all data"
					int SampleDimY;
					if (PixelFormat == PF_NV12)
					{
						SampleDimY = (DimY * 3) / 2;
					}
					else
					{
						SampleDimY = DimY;
					}

					uint32 Pitch = (DimX / GPixelFormats[PixelFormat].BlockSizeX) * GPixelFormats[PixelFormat].BlockBytes;
					TextureSample->Initialize(Data, BufferSize, FIntPoint(DimX, SampleDimY), FIntPoint(videoArea.Area.cx, videoArea.Area.cy), MediaTextureSampleFormat, Pitch, AdjustTimeStamp(FTimespan::FromMicroseconds(SampleTime / 10), EMediaTrackType::Video), FTimespan::FromMicroseconds(SampleDuration / 10));
					pBuffer->Unlock();

					TextureSample->SetPixelFormat(PixelFormat);

					VideoSampleQueue->Enqueue(TextureSample);
					UE_LOGF(LogWmfMedia, VeryVerbose, "Enqueued onto VideoSampleQueue.");
				}

			}
		}
	}
	else
	{
		if (bShowSubTypeErrorMessage)
		{
			UE_LOGF(LogWmfMedia, Log, "StreamSink %p: Unable to query MF_MT_SUBTYPE GUID of current media type", this);
			bShowSubTypeErrorMessage = false;
		}
	}
}


void FWmfMediaStreamSink::SetMediaSamplePoolAndQueue(
	TSharedPtr<FWmfMediaHardwareVideoDecodingTextureSamplePool>& InVideoSamplePool,
	TMediaSampleQueue<IMediaTextureSample>* InVideoSampleQueue,
	TFunction<FMediaTimeStamp(FTimespan, EMediaTrackType)>&& InAdjustTimeStamp,
	TFunction<bool()>&& InIsSampleStillDesired)
{
	FScopeLock Lock(&CriticalSection);
	VideoSamplePool = InVideoSamplePool;
	VideoSampleQueue = InVideoSampleQueue;
	AdjustTimeStamp = MoveTemp(InAdjustTimeStamp);
	IsSampleStillDesired = MoveTemp(InIsSampleStillDesired);
}


void FWmfMediaStreamSink::SetClockRate(float InClockRate)
{
	FScopeLock Lock(&CriticalSection);

	ClockRate = InClockRate;

	if (ClockRate != 0.0f)
	{
		bScrubbingInProgress = false;
		bSystemIsScrubbing = false;
	}

	QueueEvent(MEStreamSinkRateChanged, GUID_NULL, S_OK, NULL);
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif
