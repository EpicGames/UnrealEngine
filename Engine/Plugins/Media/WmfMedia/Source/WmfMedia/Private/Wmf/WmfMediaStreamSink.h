// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "CoreTypes.h"
#include "Containers/Queue.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "MediaSampleQueue.h"
#include "Misc/Timespan.h"
#include "RHI.h"
#include "Templates/RefCounting.h"

class FRHITexture;

class WmfMediaDecoder;
class FWmfMediaHardwareVideoDecodingTextureSamplePool;
class FWmfMediaSink;

struct ID3D11Texture2D;

/**
 * Implements a stream sink object for the WMF pipeline.
 */
class FWmfMediaStreamSink
	: public IMFGetService
	, public IMFMediaTypeHandler
	, public IMFStreamSink
{
public:

	/**
	 * Creates a stream sink for the specified major type.
	 *
	 * @param MajorType The sink's major type.
	 * @param OutSink Will contain the created sink.
	 * @return true on success, false otherwise.
	 */
	static bool Create(const GUID& MajorType, TComPtr<FWmfMediaStreamSink>& OutSink);

public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InMajorType The sink's major media type, i.e. audio or video.
	 * @param InStreamId The sink's identifier.
	 */
	FWmfMediaStreamSink(const GUID& InMajorType, DWORD InStreamId);

public:

	/**
	 * Initialize this sink.
	 *
	 * @param InOwner The media sink that owns this stream sink.
	 * @return true on success, false otherwise.
	 * @see Shutdown
	 */
	bool Initialize(FWmfMediaSink& InOwner);

	/**
	 * Call this to set which decoder we belong to.
	 *
	 * @param InDecoder
	 */
	void SetDecoder(WmfMediaDecoder* InDecoder);

	/**
	 * Pause the stream.
	 *
	 * This method is called by the owner media sink.
	 *
	 * @return Result code.
	 * @see Preroll, Restart, Start, Stop
	 */
	HRESULT Pause();

	/**
	 * Restart the stream.
	 *
	 * This method is called by the owner media sink.
	 *
	 * @return Result code.
	 * @see Pause, Preroll, Start, Stop
	 */
	HRESULT Restart();

	/**
	 * Shut down this sink.
	 *
	 * @see Initialize
	 */
	void Shutdown();

	/**
	 * Start the sink.
	 *
	 * This method is called by the owner media sink.
	 *
	 * @return Result code.
	 * @see Pause, Preroll, Restart, Stop
	 */
	HRESULT Start();

	/**
	 * Stop the stream.
	 *
	 * This method is called by the owner media sink.
	 *
	 * @return Result code.
	 * @see Pause, Preroll, Restart, Start
	 */
	HRESULT Stop();

public:

	//~ IMFGetService interface

	STDMETHODIMP GetService(__RPC__in REFGUID guidService, __RPC__in REFIID riid, __RPC__deref_out_opt LPVOID* ppvObject);

public:

	//~ IMFMediaEventGenerator interface

	STDMETHODIMP BeginGetEvent(IMFAsyncCallback* pCallback, IUnknown* pState);
	STDMETHODIMP EndGetEvent(IMFAsyncResult* pResult, IMFMediaEvent** ppEvent);
	STDMETHODIMP GetEvent(DWORD dwFlags, IMFMediaEvent** ppEvent);
	STDMETHODIMP QueueEvent(MediaEventType met, REFGUID extendedType, HRESULT hrStatus, const PROPVARIANT* pvValue);

public:

	//~ IMFMediaTypeHandler

	STDMETHODIMP GetCurrentMediaType(_Outptr_ IMFMediaType** ppMediaType);
	STDMETHODIMP GetMajorType(__RPC__out GUID* pguidMajorType);
	STDMETHODIMP GetMediaTypeByIndex(DWORD dwIndex, _Outptr_ IMFMediaType** ppType);
	STDMETHODIMP GetMediaTypeCount(__RPC__out DWORD* pdwTypeCount);
	STDMETHODIMP IsMediaTypeSupported(IMFMediaType* pMediaType, _Outptr_opt_result_maybenull_ IMFMediaType** ppMediaType);
	STDMETHODIMP SetCurrentMediaType(IMFMediaType* pMediaType);

public:

	//~ IMFStreamSink interface

	STDMETHODIMP Flush();
	STDMETHODIMP GetIdentifier(__RPC__out DWORD* pdwIdentifier);
	STDMETHODIMP GetMediaSink(__RPC__deref_out_opt IMFMediaSink** ppMediaSink);
	STDMETHODIMP GetMediaTypeHandler(__RPC__deref_out_opt IMFMediaTypeHandler** ppHandler);
	STDMETHODIMP PlaceMarker(MFSTREAMSINK_MARKER_TYPE eMarkerType, __RPC__in const PROPVARIANT* pvarMarkerValue, __RPC__in const PROPVARIANT* pvarContextValue);
	STDMETHODIMP ProcessSample(__RPC__in_opt IMFSample* pSample);

public:

	//~ IUnknown interface

	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) Release();

	/**
	 * Set current clock rate
	 *
	 * @param InClockRate Clock Rate
	 */
	void SetClockRate(float InClockRate);

	/**
	 * Set media sample pool and queue
	 *
	 * @param InVideoSamplePool sample pool to get sample from
	 * @param InVideoSampleQueue Sample queue to fill
	 * @param InSequenceIndex Pointer to current sequence index value
	 */
	void SetMediaSamplePoolAndQueue(TSharedPtr<FWmfMediaHardwareVideoDecodingTextureSamplePool>& InVideoSamplePool, TMediaSampleQueue<IMediaTextureSample>* InVideoSampleQueue, TFunction<FMediaTimeStamp(FTimespan, EMediaTrackType)>&& InAdjustTimeStamp, TFunction<bool()>&& InIsSampleStillDesired);

	/**
	 * Request more data being delivered to video output queue 
	 */
	void OnRequestMoreData();

	/**
	 * Notify instance that a seek logically started
	 */
	void OnSeekStarted();


	void OnSessionEnded();

	/**
	 * Register lambda to execute (once) as soon as any pending requests are done (if any are pending)
	 */
	bool ExecuteOnceNoPendingRequests(TFunction<void()>&& ExecuteOnIdle);

private:

	/** Hidden destructor (this class is reference counted). */
	virtual ~FWmfMediaStreamSink();

	/**
	 * Copy decoder texture to shared texture and enqueue the sample
	 *
	 * @param pSample Sample from decoder
	 */
	void CopyTextureAndEnqueueSample(IMFSample* pSample);

	/**
	 * Check if video sample queue is full
	 * @return true when Video Sample Queue is full.
	 */
	bool IsVideoSampleQueueFull() const;

private:

	/** Critical section for synchronizing access to this sink. */
	FCriticalSection CriticalSection;

	/** The event queue. */
	TComPtr<IMFMediaEventQueue> EventQueue;

	/** The sink's current media type. */
	TComPtr<IMFMediaType> CurrentMediaType;

	/** The media sink that owns this stream sink. */
	TComPtr<FWmfMediaSink> Owner;

	/** The decoder we belong to. */
	TComPtr<WmfMediaDecoder> Decoder;

	/** Holds a reference counter for this instance. */
	int32 RefCount;

	/** The stream identifier (currently fixed). */
	DWORD StreamId;

	/** The sink's major media type. */
	const GUID StreamType;

	/** ClockRate to support reverse playback */
	float ClockRate;

	/** Timer on which we wait for the next sample */
	HANDLE WaitTimer;

	/** Video sample pool from which to get next free sample */
	TSharedPtr<FWmfMediaHardwareVideoDecodingTextureSamplePool> VideoSamplePool;

	/** Video sample queue which is filled with current sample */
	TMediaSampleQueue<IMediaTextureSample>* VideoSampleQueue;

	/** Number of requests for new video samples currently in flight */
	int32 NumRequestsInFlight;
	int32 NumRequestsInFlightWithSystem;
	TFunction<void()> OnIdleAction;

	/** If true seek is in progress logically */
	bool bSeekInProgress;

	/** If true scrubbing is in progress logically */
	bool bScrubbingInProgress;

	/** If true system is in scrubbing mode (play with rate==0) */
	bool bSystemIsScrubbing;

	/** If true segment just ended, but is still logically being processed */
	bool bSegmentEnded;
	
	/** Callback to adjust given timestamp for the output queue */
	TFunction<FMediaTimeStamp(FTimespan,EMediaTrackType)> AdjustTimeStamp;

	/** Callback to check if a newly decoded sample is still desired or should be discarded. */
	TFunction<bool()> IsSampleStillDesired;

	/** Make sure we don't spam output with MF_MT_SUBTYPE error message */
	bool bShowSubTypeErrorMessage;

	/** Structure for queued media samples & markers. */
	struct FQueuedSample
	{
		/** Stream marker type. */
		MFSTREAMSINK_MARKER_TYPE MarkerType;

		/** Stream marker context. */
		PROPVARIANT* MarkerContext;

		/** The media sample. */
		TComPtr<IMFSample> Sample;
	};
};

#endif
