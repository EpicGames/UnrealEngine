// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaTopologyLoader.h"

#include "WmfMediaUtils.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Windows/AllowWindowsPlatformTypes.h"
#include <d3d11.h>
#include "Windows/HideWindowsPlatformTypes.h"

bool WmfMediaTopologyLoader::EnableHardwareAcceleration(const TComPtr<IMFTopology>& InTopology) const
{
	if (ResolveActivationNode(InTopology) == false)
	{
		return false;
	}

	return SetHardwareAccelerationOnTransformNode(InTopology);
}

bool WmfMediaTopologyLoader::ResolveActivationNode(const TComPtr<IMFTopology>& InTopology) const
{
	WORD NodeCount = 0;
	InTopology->GetNodeCount(&NodeCount);

	for (int NodeIndex = 0; NodeIndex < NodeCount; NodeIndex++)
	{
		TComPtr<IMFTopologyNode> Node;
		InTopology->GetNode(NodeIndex, &Node);

		MF_TOPOLOGY_TYPE NodeType;
		Node->GetNodeType(&NodeType);

		if (NodeType != MF_TOPOLOGY_OUTPUT_NODE)
		{
			continue;
		}

		TComPtr<IUnknown> UnknownObject;
		Node->GetObject(&UnknownObject);

		TComPtr<IMFStreamSink> ExistingStreamSink;
		UnknownObject->QueryInterface(_uuidof(IMFStreamSink), (void**)&ExistingStreamSink);

		// Already a StreamSink
		if (ExistingStreamSink)
		{
			continue;
		}

		// Otherwise convert IMFActivate to IMFStreamSink
		TComPtr<IMFActivate> Activate;
		UnknownObject->QueryInterface(_uuidof(IMFActivate), (void**)&Activate);

		if (Activate)
		{
			TComPtr<IMFMediaSink> MediaSink;
			Activate->ActivateObject(IID_IMFMediaSink, (void**)&MediaSink);

			if (MediaSink)
			{
				UINT32 StreamID;
				if (Node->GetUINT32(MF_TOPONODE_STREAMID, &StreamID) != S_OK)
				{
					StreamID = 0;
				}
				
				TComPtr<IMFStreamSink> StreamSink;
				MediaSink->GetStreamSinkById(StreamID, &StreamSink);

				if (StreamSink)
				{
					Node->SetObject(StreamSink);
				}
				else
				{
					MediaSink->AddStreamSink(StreamID, nullptr, &StreamSink);
					if (StreamSink)
					{
						Node->SetObject(StreamSink);
					}
					else
					{
						UE_LOGF(LogWmfMedia, Error, "Output Node not configured correctly");
						return false;
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogWmfMedia, Error, "Output node not IMFStreamSink nor IMFActivate");
			return false;
		}
	}
	return true;
}


bool WmfMediaTopologyLoader::SetHardwareAccelerationOnTransformNode(const TComPtr<IMFTopology>& InTopology) const
{
	TComPtr<IMFTopologyNode> D3DManagerNode;
	TComPtr<IMFDXGIDeviceManager> D3DManager;
	TComPtr<IMFTransform> Transform;

	if (FindDeviceManager(InTopology, D3DManager, D3DManagerNode, Transform))
	{
		ULONG_PTR ptr = (ULONG_PTR)((IMFDXGIDeviceManager*)D3DManager);
		HRESULT hr = Transform->ProcessMessage(MFT_MESSAGE_SET_D3D_MANAGER, ptr);
		if (SUCCEEDED(hr))
		{
			return true;
		}
		else if (hr == E_FAIL)
		{
			return false;
		}
	}

	return false;
}

bool WmfMediaTopologyLoader::FindDeviceManager(const TComPtr<IMFTopology>& InTopology, TComPtr<IMFDXGIDeviceManager>& OutDeviceManager, TComPtr<IMFTopologyNode>& OutD3DManagerNode, TComPtr<IMFTransform>& OutTransformNode) const
{
	WORD NodeCount = 0;
	InTopology->GetNodeCount(&NodeCount);

	for (int NodeIndex = 0; NodeIndex < NodeCount; NodeIndex++)
	{
		TComPtr<IMFTopologyNode> Node;
		InTopology->GetNode(NodeIndex, &Node);

		MF_TOPOLOGY_TYPE NodeType;
		Node->GetNodeType(&NodeType);

		if (NodeType == MF_TOPOLOGY_TRANSFORM_NODE)
		{
			TComPtr<IUnknown> Unknown;
			Node->GetObject(&Unknown);

			if (Unknown)
			{
				TComPtr<IMFTransform> Transform;
				Unknown->QueryInterface(IID_IMFTransform, (void**)&Transform);
				if (Transform)
				{
					TComPtr<IMFAttributes> Attributes;
					Transform->GetAttributes(&Attributes);
					if (Attributes)
					{
						UINT32 D3D11_Aware = 0;
#if (WINVER < _WIN32_WINNT_WIN8) 
						const GUID MF_SA_D3D11_AWARE = { 0x206b4fc8, 0xfcf9, 0x4c51, { 0xaf, 0xe3, 0x97, 0x64, 0x36, 0x9e, 0x33, 0xa0 } };
#endif
						if (Attributes->GetUINT32(MF_SA_D3D11_AWARE, &D3D11_Aware) == S_OK)
						{
							DWORD OutputCount = 0;
							Node->GetOutputCount(&OutputCount);

							TComPtr<IMFTopologyNode> DeviceManagerNode;
							DWORD DeviceManagerIndex = 0;
							Node->GetOutput(0, &DeviceManagerNode, &DeviceManagerIndex);

							TComPtr<IMFDXGIDeviceManager> DeviceManager;

							TComPtr<IUnknown> DeviceManagerUnknown;
							DeviceManagerNode->GetObject(&DeviceManagerUnknown);

							TComPtr<IMFStreamSink> DeviceManagerStreamSink;
							DeviceManagerUnknown->QueryInterface(__uuidof(IMFStreamSink), (void**)&DeviceManagerStreamSink);

							if (DeviceManagerStreamSink.IsValid())
							{
								HRESULT hr = MFGetService(DeviceManagerStreamSink, MR_VIDEO_ACCELERATION_SERVICE, IID_PPV_ARGS(&DeviceManager));

								ULONG_PTR ptr = (ULONG_PTR)((IMFDXGIDeviceManager*)DeviceManager);

								if (DeviceManager.IsValid())
								{
									OutDeviceManager = DeviceManager;
									OutD3DManagerNode = Node;
									OutTransformNode = Transform;
									return true;
								}
							}
						}
					}
				}
			}
		}
	}

	return false;
}

#endif
