// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

/**
 * Custom topology loader used to check if video will be decoded on GPU or CPU
 * even when hardware acceleration is enabled.
 */
class WmfMediaTopologyLoader
{
public:

	bool EnableHardwareAcceleration(const TComPtr<IMFTopology>& InTopology) const;

private:

	bool ResolveActivationNode(const TComPtr<IMFTopology>& InTopology) const;
	bool SetHardwareAccelerationOnTransformNode(const TComPtr<IMFTopology>& InTopology) const;
	bool FindDeviceManager(const TComPtr<IMFTopology>& InTopology, TComPtr<IMFDXGIDeviceManager>& OutDeviceManager, TComPtr<IMFTopologyNode>& OutD3DManagerNode, TComPtr<IMFTransform>& OutTransformNode) const;
};

#endif
