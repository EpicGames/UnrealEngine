// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaUtils.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"

#include "RHI.h"
#include "DynamicRHI.h"

#include "IWmfMediaModule.h"
#include "Math/NumericLimits.h"
#include "Misc/FileHelper.h"
#include "Serialization/Archive.h"
#include "Serialization/ArrayReader.h"

#include "WmfMediaByteStream.h"

#include "WmfMediaCodec/WmfMediaCodecManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"

#define WMFMEDIA_FRAMERATELUT_SIZE 9

// MPEG-2 audio sub-types
#define OTHER_FORMAT_MPEG2_AC3				0xe06d80e4 // MPEG-2 AC3
#define OTHER_FORMAT_MPEG2_AUDIO			0xe06d802b // MPEG-2 Audio
#define OTHER_FORMAT_MPEG2_DOLBY_AC3		0xe06d802c // Dolby AC3
#define OTHER_FORMAT_MPEG2_DTS				0xe06d8033 // MPEG-2 DTS
#define OTHER_FORMAT_MPEG2_LPCM_AUDIO		0xe06d8032 // DVD LPCM Audio
#define OTHER_FORMAT_MPEG2_SDDS				0xe06d8034 // SDDS

// MPEG-2 video sub-types
#define OTHER_FORMAT_MPEG2_DVD_SUBPICTURE	0xe06d802d // DVD Sub-picture
#define OTHER_FORMAT_MPEG2_VIDEO			0xe06d80e3 // MPEG-2 Video


/* Local constants & helpers
 *****************************************************************************/

namespace WmfMedia
{
	// common media formats not defined by WMF
	const GUID OtherFormatMpeg2_Base = { 0x00000000, 0xdb46, 0x11cf, { 0xb4, 0xd1, 0x00, 0x80, 0x05f, 0x6c, 0xbb, 0xea } };
	const GUID OtherVideoFormat_LifeCam = { 0x3032344d, 0x0000, 0x0010, { 0x80, 0x00, 0x00, 0xAA, 0x00, 0x38, 0x9B, 0x71 } }; // Microsoft LifeCam UVC 1.0 video
	const GUID OtherVideoFormat_QuickTime = { 0x61766331, 0x767a, 0x494d, { 0xb4, 0x78, 0xf2, 0x9d, 0x25, 0xdc, 0x90, 0x37 } }; // 1cva


	/** Lookup table for frame rates that are handled specially in WMF. */
	const struct FFrameRateLut
	{
		float FrameRate;
		int32 Numerator;
		int32 Denominator;
		int32 DurationTicks;
	}
	FrameRateLut[WMFMEDIA_FRAMERATELUT_SIZE] = {
		{ 59.95f, 60000, 1001, 166833 },
		{ 29.97f, 30000, 1001, 333667 },
		{ 23.976f, 24000, 1001, 417188 },
		{ 60.0f, 60, 1, 166667 },
		{ 30.0f, 30, 1, 333333 },
		{ 50.0f, 50, 1, 200000 },
		{ 25.0f, 25, 1, 400000 },
		{ 24.0f, 24, 1, 416667 },
		{ 0.0f, 0, 1, 0 }
	};

	/** List of supported major media types. */
	GUID const* const SupportedMajorTypes[] = {
		&MFMediaType_Audio,
		&MFMediaType_Binary,
		&MFMediaType_SAMI,
		&MFMediaType_Video
	};

	/** List of supported audio channel counts (in order of preference). */
	const DWORD SupportedAudioChannels[] = { 1, 2, 3, 4, 5, 6, 7, 8 };

	/** List of supported audio types (in order of preference). */
	const struct FAudioFormat
	{
		GUID const* const SubType;
		UINT32 BitsPerSample;
	}
	SupportedAudioFormats[] = {
		{ &MFAudioFormat_Float, 32u },
		{ &MFAudioFormat_Float, 64u },
		{ &MFAudioFormat_PCM, 32u },
		{ &MFAudioFormat_PCM, 16u },
		{ &MFAudioFormat_PCM, 8u }
	};

	/** List of supported video media types (in order of preference). */
	GUID const* const SupportedVideoFormats[] =
	{
		// uncompressed
//		&MFVideoFormat_A2R10G10B10,
//		&MFVideoFormat_A16B16G16R16F,
		&MFVideoFormat_ARGB32,
//		&MFVideoFormat_RGB32,
//		&MFVideoFormat_RGB24,
//		&MFVideoFormat_RGB555,
//		&MFVideoFormat_RGB565,
		&MFVideoFormat_RGB8,

		// 8-bit YUV (packed)
//		&MFVideoFormat_AYUV,
//		&MFVideoFormat_UYVY,
//		&MFVideoFormat_YUY2,
//		&MFVideoFormat_YVYU,

		// 8-bit YUV (planar)
//		&MFVideoFormat_I420,
//		&MFVideoFormat_IYUV,
		&MFVideoFormat_NV12,
//		&MFVideoFormat_YV12,

		// 10-bit YUV (packed)
//		&MFVideoFormat_v210,
//		&MFVideoFormat_v410,
//		&MFVideoFormat_Y210,
//		&MFVideoFormat_Y410,

		// 10-bit YUV (planar)
//		&MFVideoFormat_P010,
//		&MFVideoFormat_P210,

		// 16-bit YUV (packed)
//		&MFVideoFormat_v216,
//		&MFVideoFormat_Y216,
		&MFVideoFormat_Y416,

		// 16-bit YUV (planar)
//		&MFVideoFormat_P016,
//		&MFVideoFormat_P216,

		&MFVideoFormat_L8,
		&MFVideoFormat_L16,
		&MFVideoFormat_D16,

	};


	/** List of supported audio types. */
	TArray<TComPtr<IMFMediaType>> SupportedAudioTypes;

	/** List of supported binary types. */
	TArray<TComPtr<IMFMediaType>> SupportedBinaryTypes;

	/** List of supported SAMI types. */
	TArray<TComPtr<IMFMediaType>> SupportedSamiTypes;

	/** List of supported video types. */
	TArray<TComPtr<IMFMediaType>> SupportedVideoTypes;


	/** Initialize the lists of supported media types. */
	void InitializeSupportedTypes()
	{
		static bool Initialized = false;

		if (Initialized)
		{
			return;
		}

		Initialized = true;

		// initialize audio types
		for (const FAudioFormat& Format : SupportedAudioFormats)
		{
			for (UINT32 NumChannels : SupportedAudioChannels)
			{
				TComPtr<IMFMediaType> MediaType;

				if (FAILED(::MFCreateMediaType(&MediaType)) ||
					FAILED(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio)) ||
					FAILED(MediaType->SetGUID(MF_MT_SUBTYPE, *Format.SubType)) ||
					FAILED(MediaType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE)) ||
					FAILED(MediaType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, Format.BitsPerSample)) ||
					FAILED(MediaType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, NumChannels)))
				{
					UE_LOGF(LogWmfMedia, Error, "Failed to initialize supported audio type: %ls, %i bits/sample, %i channels", *SubTypeToString(*Format.SubType), Format.BitsPerSample, NumChannels);
				}

				SupportedAudioTypes.Add(MediaType);
			}
		}

		// initialize binary types
		{
			TComPtr<IMFMediaType> MediaType;

			if (FAILED(::MFCreateMediaType(&MediaType)) ||
				FAILED(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Binary)) ||
				FAILED(MediaType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE)))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to initialize supported binary types");
			}

			SupportedBinaryTypes.Add(MediaType);
		}

		// initialize SAMI types
		{
			TComPtr<IMFMediaType> MediaType;

			if (FAILED(::MFCreateMediaType(&MediaType)) ||
				FAILED(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_SAMI)) ||
				FAILED(MediaType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE)))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to initialize supported SAMI types");
			}

			SupportedSamiTypes.Add(MediaType);
		}

		// initialize video types
		for (GUID const* const Format : SupportedVideoFormats)
		{
			TComPtr<IMFMediaType> MediaType;

			if (FAILED(::MFCreateMediaType(&MediaType)) ||
				FAILED(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video)) ||
				FAILED(MediaType->SetGUID(MF_MT_SUBTYPE, *Format)) ||
				FAILED(MediaType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE)) ||
				FAILED(MediaType->SetUINT32(MF_MT_INTERLACE_MODE, (UINT32)MFVideoInterlace_Progressive)))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to initialize supported video type: %ls", *SubTypeToString(*Format));
			}

			SupportedVideoTypes.Add(MediaType);
		}
	}
}


/* WmfMedia functions
 *****************************************************************************/

namespace WmfMedia
{
	FString AttributeToString(const GUID& Guid)
	{
		if (Guid == MF_MT_MAJOR_TYPE) return TEXT("MF_MT_MAJOR_TYPE");
		if (Guid == MF_MT_MAJOR_TYPE) return TEXT("MF_MT_MAJOR_TYPE");
		if (Guid == MF_MT_SUBTYPE) return TEXT("MF_MT_SUBTYPE");
		if (Guid == MF_MT_ALL_SAMPLES_INDEPENDENT) return TEXT("MF_MT_ALL_SAMPLES_INDEPENDENT");
		if (Guid == MF_MT_FIXED_SIZE_SAMPLES) return TEXT("MF_MT_FIXED_SIZE_SAMPLES");
		if (Guid == MF_MT_COMPRESSED) return TEXT("MF_MT_COMPRESSED");
		if (Guid == MF_MT_SAMPLE_SIZE) return TEXT("MF_MT_SAMPLE_SIZE");
		if (Guid == MF_MT_WRAPPED_TYPE) return TEXT("MF_MT_WRAPPED_TYPE");
		if (Guid == MF_MT_AUDIO_NUM_CHANNELS) return TEXT("MF_MT_AUDIO_NUM_CHANNELS");
		if (Guid == MF_MT_AUDIO_SAMPLES_PER_SECOND) return TEXT("MF_MT_AUDIO_SAMPLES_PER_SECOND");
		if (Guid == MF_MT_AUDIO_FLOAT_SAMPLES_PER_SECOND) return TEXT("MF_MT_AUDIO_FLOAT_SAMPLES_PER_SECOND");
		if (Guid == MF_MT_AUDIO_AVG_BYTES_PER_SECOND) return TEXT("MF_MT_AUDIO_AVG_BYTES_PER_SECOND");
		if (Guid == MF_MT_AUDIO_BLOCK_ALIGNMENT) return TEXT("MF_MT_AUDIO_BLOCK_ALIGNMENT");
		if (Guid == MF_MT_AUDIO_BITS_PER_SAMPLE) return TEXT("MF_MT_AUDIO_BITS_PER_SAMPLE");
		if (Guid == MF_MT_AUDIO_VALID_BITS_PER_SAMPLE) return TEXT("MF_MT_AUDIO_VALID_BITS_PER_SAMPLE");
		if (Guid == MF_MT_AUDIO_SAMPLES_PER_BLOCK) return TEXT("MF_MT_AUDIO_SAMPLES_PER_BLOCK");
		if (Guid == MF_MT_AUDIO_CHANNEL_MASK) return TEXT("MF_MT_AUDIO_CHANNEL_MASK");
		if (Guid == MF_MT_AUDIO_FOLDDOWN_MATRIX) return TEXT("MF_MT_AUDIO_FOLDDOWN_MATRIX");
		if (Guid == MF_MT_AUDIO_WMADRC_PEAKREF) return TEXT("MF_MT_AUDIO_WMADRC_PEAKREF");
		if (Guid == MF_MT_AUDIO_WMADRC_PEAKTARGET) return TEXT("MF_MT_AUDIO_WMADRC_PEAKTARGET");
		if (Guid == MF_MT_AUDIO_WMADRC_AVGREF) return TEXT("MF_MT_AUDIO_WMADRC_AVGREF");
		if (Guid == MF_MT_AUDIO_WMADRC_AVGTARGET) return TEXT("MF_MT_AUDIO_WMADRC_AVGTARGET");
		if (Guid == MF_MT_AUDIO_PREFER_WAVEFORMATEX) return TEXT("MF_MT_AUDIO_PREFER_WAVEFORMATEX");
		if (Guid == MF_MT_AAC_PAYLOAD_TYPE) return TEXT("MF_MT_AAC_PAYLOAD_TYPE");
		if (Guid == MF_MT_AAC_AUDIO_PROFILE_LEVEL_INDICATION) return TEXT("MF_MT_AAC_AUDIO_PROFILE_LEVEL_INDICATION");
		if (Guid == MF_MT_FRAME_SIZE) return TEXT("MF_MT_FRAME_SIZE");
		if (Guid == MF_MT_FRAME_RATE) return TEXT("MF_MT_FRAME_RATE");
		if (Guid == MF_MT_FRAME_RATE_RANGE_MAX) return TEXT("MF_MT_FRAME_RATE_RANGE_MAX");
		if (Guid == MF_MT_FRAME_RATE_RANGE_MIN) return TEXT("MF_MT_FRAME_RATE_RANGE_MIN");
		if (Guid == MF_MT_PIXEL_ASPECT_RATIO) return TEXT("MF_MT_PIXEL_ASPECT_RATIO");
		if (Guid == MF_MT_DRM_FLAGS) return TEXT("MF_MT_DRM_FLAGS");
		if (Guid == MF_MT_PAD_CONTROL_FLAGS) return TEXT("MF_MT_PAD_CONTROL_FLAGS");
		if (Guid == MF_MT_SOURCE_CONTENT_HINT) return TEXT("MF_MT_SOURCE_CONTENT_HINT");
		if (Guid == MF_MT_VIDEO_CHROMA_SITING) return TEXT("MF_MT_VIDEO_CHROMA_SITING");
		if (Guid == MF_MT_INTERLACE_MODE) return TEXT("MF_MT_INTERLACE_MODE");
		if (Guid == MF_MT_TRANSFER_FUNCTION) return TEXT("MF_MT_TRANSFER_FUNCTION");
		if (Guid == MF_MT_VIDEO_PRIMARIES) return TEXT("MF_MT_VIDEO_PRIMARIES");
		if (Guid == MF_MT_CUSTOM_VIDEO_PRIMARIES) return TEXT("MF_MT_CUSTOM_VIDEO_PRIMARIES");
		if (Guid == MF_MT_YUV_MATRIX) return TEXT("MF_MT_YUV_MATRIX");
		if (Guid == MF_MT_VIDEO_LIGHTING) return TEXT("MF_MT_VIDEO_LIGHTING");
		if (Guid == MF_MT_VIDEO_NOMINAL_RANGE) return TEXT("MF_MT_VIDEO_NOMINAL_RANGE");
		if (Guid == MF_MT_GEOMETRIC_APERTURE) return TEXT("MF_MT_GEOMETRIC_APERTURE");
		if (Guid == MF_MT_MINIMUM_DISPLAY_APERTURE) return TEXT("MF_MT_MINIMUM_DISPLAY_APERTURE");
		if (Guid == MF_MT_PAN_SCAN_APERTURE) return TEXT("MF_MT_PAN_SCAN_APERTURE");
		if (Guid == MF_MT_PAN_SCAN_ENABLED) return TEXT("MF_MT_PAN_SCAN_ENABLED");
		if (Guid == MF_MT_AVG_BITRATE) return TEXT("MF_MT_AVG_BITRATE");
		if (Guid == MF_MT_AVG_BIT_ERROR_RATE) return TEXT("MF_MT_AVG_BIT_ERROR_RATE");
		if (Guid == MF_MT_MAX_KEYFRAME_SPACING) return TEXT("MF_MT_MAX_KEYFRAME_SPACING");
		if (Guid == MF_MT_DEFAULT_STRIDE) return TEXT("MF_MT_DEFAULT_STRIDE");
		if (Guid == MF_MT_PALETTE) return TEXT("MF_MT_PALETTE");
		if (Guid == MF_MT_USER_DATA) return TEXT("MF_MT_USER_DATA");
		if (Guid == MF_MT_AM_FORMAT_TYPE) return TEXT("MF_MT_AM_FORMAT_TYPE");
		if (Guid == MF_MT_MPEG_START_TIME_CODE) return TEXT("MF_MT_MPEG_START_TIME_CODE");
		if (Guid == MF_MT_MPEG2_PROFILE) return TEXT("MF_MT_MPEG2_PROFILE");
		if (Guid == MF_MT_MPEG2_LEVEL) return TEXT("MF_MT_MPEG2_LEVEL");
		if (Guid == MF_MT_MPEG2_FLAGS) return TEXT("MF_MT_MPEG2_FLAGS");
		if (Guid == MF_MT_MPEG_SEQUENCE_HEADER) return TEXT("MF_MT_MPEG_SEQUENCE_HEADER");
		if (Guid == MF_MT_DV_AAUX_SRC_PACK_0) return TEXT("MF_MT_DV_AAUX_SRC_PACK_0");
		if (Guid == MF_MT_DV_AAUX_CTRL_PACK_0) return TEXT("MF_MT_DV_AAUX_CTRL_PACK_0");
		if (Guid == MF_MT_DV_AAUX_SRC_PACK_1) return TEXT("MF_MT_DV_AAUX_SRC_PACK_1");
		if (Guid == MF_MT_DV_AAUX_CTRL_PACK_1) return TEXT("MF_MT_DV_AAUX_CTRL_PACK_1");
		if (Guid == MF_MT_DV_VAUX_SRC_PACK) return TEXT("MF_MT_DV_VAUX_SRC_PACK");
		if (Guid == MF_MT_DV_VAUX_CTRL_PACK) return TEXT("MF_MT_DV_VAUX_CTRL_PACK");
		if (Guid == MF_MT_ARBITRARY_HEADER) return TEXT("MF_MT_ARBITRARY_HEADER");
		if (Guid == MF_MT_ARBITRARY_FORMAT) return TEXT("MF_MT_ARBITRARY_FORMAT");
		if (Guid == MF_MT_IMAGE_LOSS_TOLERANT) return TEXT("MF_MT_IMAGE_LOSS_TOLERANT");
		if (Guid == MF_MT_MPEG4_SAMPLE_DESCRIPTION) return TEXT("MF_MT_MPEG4_SAMPLE_DESCRIPTION");
		if (Guid == MF_MT_MPEG4_CURRENT_SAMPLE_ENTRY) return TEXT("MF_MT_MPEG4_CURRENT_SAMPLE_ENTRY");
		if (Guid == MF_MT_ORIGINAL_4CC) return TEXT("MF_MT_ORIGINAL_4CC");
		if (Guid == MF_MT_ORIGINAL_WAVE_FORMAT_TAG) return TEXT("MF_MT_ORIGINAL_WAVE_FORMAT_TAG");

		// unknown identifier
		return GuidToString(Guid);
	}


	FString CaptureDeviceRoleToString(ERole Role)
	{
		switch (Role)
		{
		case eCommunications: return TEXT("Communications");
		case eConsole: return TEXT("Console");
		case eMultimedia: return TEXT("Multimedia");

		default:
			return TEXT("Unknown");
		}
	}


	HRESULT CopyAttribute(IMFAttributes* Src, IMFAttributes* Dest, const GUID& Key)
	{
		PROPVARIANT var;
		PropVariantInit(&var);

		HRESULT Result = Src->GetItem(Key, &var);
		
		if (SUCCEEDED(Result))
		{
			Result = Dest->SetItem(Key, var);
			PropVariantClear(&var);
		}

		return Result;
	}


	bool IsFormatFamily(const GUID& lhs, const GUID& rhs)
	{
		return FMemory::Memcmp(&lhs.Data2, &rhs.Data2, 12) == 0; //-V512
	}

	TComPtr<IMFMediaType> CreateOutputType(IMFMediaType& InputType, bool AllowNonStandardCodecs, bool IsVideoDevice)
	{
		GUID MajorType;
		{
			const HRESULT Result = InputType.GetGUID(MF_MT_MAJOR_TYPE, &MajorType);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to get major type: %ls", *ResultToString(Result));
				return NULL;
			}
		}

		GUID SubType;
		{
			const HRESULT Result = InputType.GetGUID(MF_MT_SUBTYPE, &SubType);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to get sub-type: %ls", *ResultToString(Result));
				return NULL;
			}
		}

		TComPtr<IMFMediaType> OutputType;
		{
			HRESULT Result = ::MFCreateMediaType(&OutputType);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to create %ls output type: %ls", *MajorTypeToString(MajorType), *ResultToString(Result));
				return NULL;
			}

			Result = OutputType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to initialize %ls output type: %ls", *MajorTypeToString(MajorType), *ResultToString(Result));
				return NULL;
			}
		}

		if (MajorType == MFMediaType_Audio)
		{
			// filter unsupported audio formats
			if (IsFormatFamily(SubType, MFMPEG4Format_Base))
			{
				if (AllowNonStandardCodecs)
				{
					UE_LOGF(LogWmfMedia, Verbose, "Allowing non-standard MP4 audio type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
				}
				else
				{
					const bool DocumentedFormat =
						(SubType.Data1 == WAVE_FORMAT_ADPCM) ||
						(SubType.Data1 == WAVE_FORMAT_ALAW) ||
						(SubType.Data1 == WAVE_FORMAT_MULAW) ||
						(SubType.Data1 == WAVE_FORMAT_IMA_ADPCM) ||
						(SubType.Data1 == MFAudioFormat_AAC.Data1) ||
						(SubType.Data1 == MFAudioFormat_MP3.Data1) ||
						(SubType.Data1 == MFAudioFormat_PCM.Data1);

					const bool UndocumentedFormat =
						(SubType.Data1 == WAVE_FORMAT_WMAUDIO2) ||
						(SubType.Data1 == WAVE_FORMAT_WMAUDIO3) ||
						(SubType.Data1 == WAVE_FORMAT_WMAUDIO_LOSSLESS);

					if (!DocumentedFormat && !UndocumentedFormat)
					{
						UE_LOGF(LogWmfMedia, Warning, "Skipping non-standard MP4 audio type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
						return NULL;
					}
				}
			}
			else if (!IsFormatFamily(SubType, MFAudioFormat_Base))
			{
				if (AllowNonStandardCodecs)
				{
					UE_LOGF(LogWmfMedia, Verbose, "Allowing non-standard audio type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
				}
				else
				{
					UE_LOGF(LogWmfMedia, Warning, "Skipping non-standard audio type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
					return NULL;
				}
			}

			// configure audio output
			if (FAILED(OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio)) ||
				FAILED(OutputType->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_PCM)) ||
				FAILED(OutputType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16u)))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to initialize audio output type");
				return NULL;
			}

			// copy media type attributes
			if (FAILED(CopyAttribute(&InputType, OutputType, MF_MT_AUDIO_NUM_CHANNELS)) ||
				FAILED(CopyAttribute(&InputType, OutputType, MF_MT_AUDIO_SAMPLES_PER_SECOND)))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to copy audio output type attributes");
				return NULL;
			}
		}
		else if (MajorType == MFMediaType_Binary)
		{
			const HRESULT Result = OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Binary);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to initialize binary output type: %ls", *ResultToString(Result));
				return NULL;
			}
		}
		else if (MajorType == MFMediaType_SAMI)
		{
			// configure caption output
			const HRESULT Result = OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_SAMI);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to initialize caption output type: %ls", *ResultToString(Result));
				return NULL;
			}
		}
		else if (MajorType == MFMediaType_Video)
		{
			// filter unsupported video types
			if (!IsFormatFamily(SubType, MFVideoFormat_Base))
			{
				if (AllowNonStandardCodecs)
				{
					UE_LOGF(LogWmfMedia, Verbose, "Allowing non-standard video type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
				}
				else
				{
					UE_LOGF(LogWmfMedia, Warning, "Skipping non-standard video type %ls (%ls) \"%ls\"", *SubTypeToString(SubType), *GuidToString(SubType), *FourccToString(SubType.Data1));
					return NULL;
				}
			}

			if ((SubType == MFVideoFormat_H264) || (SubType == MFVideoFormat_H264_ES))
			{
				if (IsVideoDevice /*&& !FPlatformMisc::VerifyWindowsVersion(6, 2)*/ /*Win8*/)
				{
					UE_LOGF(LogWmfMedia, Warning, "H264 video type requires Windows 8 or newer (your version is %ls)", *FPlatformMisc::GetOSVersion());
					return NULL;
				}
			}

			if ((SubType == MFVideoFormat_HEVC) || (SubType == MFVideoFormat_HEVC_ES))
			{
				if (!FPlatformMisc::VerifyWindowsVersion(10, 0) /*Win10*/)
				{
					if (!FPlatformMisc::VerifyWindowsVersion(6, 2) /*Win8*/)
					{
						UE_LOGF(LogWmfMedia, Warning, "HEVC video type requires Windows 10 or newer (your version is %ls)", *FPlatformMisc::GetOSVersion());
						return NULL;
					}

					UE_LOGF(LogWmfMedia, Warning, "HEVC video type requires Windows 10 or newer (your version is %ls), and game must be manifested for Windows 10", *FPlatformMisc::GetOSVersion());
				}
			}

			// configure video output
			HRESULT Result = OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to set video output type: %ls", *ResultToString(Result));
				return NULL;
			}

			UE_LOGF(LogWmfMedia, Verbose, "SubType: { 0x%08X, 0x%04X, 0x%04X, { 0x%02X, 0x%02X, 0x%02X, 0x%02X, 0x%02X, 0x%02X, 0x%02X, 0x%02X } };",
				SubType.Data1,
				SubType.Data2,
				SubType.Data3,
				SubType.Data4[0],
				SubType.Data4[1],
				SubType.Data4[2],
				SubType.Data4[3],
				SubType.Data4[4],
				SubType.Data4[5],
				SubType.Data4[6],
				SubType.Data4[7]); 


			IWmfMediaModule* WmfMediaModule = IWmfMediaModule::Get();
			if (WmfMediaModule && WmfMediaModule->GetCodecManager()->IsCodecSupported(MajorType, SubType))
			{
				GUID OutputSubType;
				WmfMediaModule->GetCodecManager()->SetVideoFormat(SubType, OutputSubType);
				OutputType->SetGUID(MF_MT_SUBTYPE, OutputSubType);
			}
			else if ((SubType == MFVideoFormat_HEVC) ||
					 (SubType == MFVideoFormat_HEVC_ES) ||
					 (SubType == MFVideoFormat_NV12) ||
					 (SubType == MFVideoFormat_IYUV))
			{
				Result = OutputType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_NV12);
			}
			else
			{
				const bool Uncompressed =
					(SubType == MFVideoFormat_RGB555) ||
					(SubType == MFVideoFormat_RGB565) ||
					(SubType == MFVideoFormat_RGB24) ||
					(SubType == MFVideoFormat_RGB32) ||
					(SubType == MFVideoFormat_ARGB32);

				Result = OutputType->SetGUID(MF_MT_SUBTYPE, Uncompressed ? MFVideoFormat_RGB32 : MFVideoFormat_YUY2);
			}

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Warning, "Failed to set video output sub-type: %ls", *ResultToString(Result));
				return NULL;
			}

			// copy media type attributes
			if (IsVideoDevice)
			{
				// the following attributes seem to help with web cam issues on Windows 7,
				// but we generally don't want to copy these for any other media sources
				// and let the WMF topology resolver pick optimal defaults instead.

				if (FAILED(CopyAttribute(&InputType, OutputType, MF_MT_FRAME_RATE)) ||
					FAILED(CopyAttribute(&InputType, OutputType, MF_MT_FRAME_SIZE)))
				{
					UE_LOGF(LogWmfMedia, Warning, "Failed to copy video output type attributes");
					return NULL;
				}
			}
		}
		else
		{
			return NULL; // unsupported input type
		}

		return OutputType;
	}


	FString DumpAttributes(IMFAttributes& Attributes)
	{
		FString Dump;

		UINT32 NumAttributes = 0;
		{
			const HRESULT Result = Attributes.GetCount(&NumAttributes);

			if (FAILED(Result))
			{
				Dump += FString::Printf(TEXT("\tFailed to get attribute count: %s\n"), *ResultToString(Result));
				return Dump;
			}
		}

		for (UINT32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
		{
			GUID Guid = { 0 };
			PROPVARIANT Item;
			{
				::PropVariantInit(&Item);

				const HRESULT Result = Attributes.GetItemByIndex(AttributeIndex, &Guid, &Item);

				if (FAILED(Result))
				{
					Dump += FString::Printf(TEXT("\tFailed to get attribute %i: %s\n"), AttributeIndex, *ResultToString(Result));
				}
			}

			const FString GuidName = AttributeToString(Guid);

			if (Guid == MF_MT_AM_FORMAT_TYPE)
			{
				Dump += FString::Printf(TEXT("\t%s: %s (%s)\n"), *GuidName, *GuidToString(*Item.puuid), *FormatTypeToString(*Item.puuid));
			}
			else if (Guid == MF_MT_MAJOR_TYPE)
			{
				Dump += FString::Printf(TEXT("\t%s: %s (%s)\n"), *GuidName, *GuidToString(*Item.puuid), *MajorTypeToString(*Item.puuid));
			}
			else if (Guid == MF_MT_SUBTYPE)
			{
				Dump += FString::Printf(TEXT("\t%s: %s (%s)\n"), *GuidName, *GuidToString(*Item.puuid), *SubTypeToString(*Item.puuid));
			}
			else if ((Guid == MF_MT_FRAME_RATE) || (Guid == MF_MT_FRAME_RATE_RANGE_MAX) || (Guid == MF_MT_FRAME_RATE_RANGE_MIN))
			{
				UINT32 High = 0;
				UINT32 Low = 0;

				::Unpack2UINT32AsUINT64(Item.uhVal.QuadPart, &High, &Low);
				Dump += FString::Printf(TEXT("\t%s: %d/%d\n"), *GuidName, High, Low);
			}
			else if (Guid == MF_MT_FRAME_SIZE)
			{
				UINT32 High = 0;
				UINT32 Low = 0;

				::Unpack2UINT32AsUINT64(Item.uhVal.QuadPart, &High, &Low);
				Dump += FString::Printf(TEXT("\t%s: %d x %d\n"), *GuidName, High, Low);
			}
			else if (Guid == MF_MT_INTERLACE_MODE)
			{
				Dump += FString::Printf(TEXT("\t%s: %lu (%s)\n"), *GuidName, Item.ulVal, *InterlaceModeToString((MFVideoInterlaceMode)Item.ulVal));
			}
			else if (Guid == MF_MT_PIXEL_ASPECT_RATIO)
			{
				UINT32 High = 0;
				UINT32 Low = 0;

				::Unpack2UINT32AsUINT64(Item.uhVal.QuadPart, &High, &Low);
				Dump += FString::Printf(TEXT("\t%s: %d:%d\n"), *GuidName, High, Low);
			}
			else if ((Guid == MF_MT_GEOMETRIC_APERTURE) || (Guid == MF_MT_MINIMUM_DISPLAY_APERTURE) || (Guid == MF_MT_PAN_SCAN_APERTURE))
			{
				if (Item.caub.cElems < sizeof(MFVideoArea))
				{
					Dump += FString::Printf(TEXT("\t%s: failed to get value (buffer too small)\n"), *GuidName);
				}
				else
				{
					MFVideoArea* Area = (MFVideoArea*)Item.caub.pElems;

					Dump += FString::Printf(TEXT("\t%s: (%f,%f) (%ld,%ld)\n"),
						*GuidName,
						Area->OffsetX.value + (static_cast<float>(Area->OffsetX.fract) / 65536.0f),
						Area->OffsetY.value + (static_cast<float>(Area->OffsetY.fract) / 65536.0f),
						Area->Area.cx,
						Area->Area.cy);
				}
			}
			else
			{
				switch (Item.vt)
				{
				case VT_UI4:
					Dump += FString::Printf(TEXT("\t%s: %lu\n"), *GuidName, Item.ulVal);
					break;

				case VT_UI8:
					Dump += FString::Printf(TEXT("\t%s: %llu\n"), *GuidName, Item.uhVal.QuadPart);
					break;

				case VT_R8:
					Dump += FString::Printf(TEXT("\t%s: %f\n"), *GuidName, Item.dblVal);
					break;

				case VT_CLSID:
					Dump += FString::Printf(TEXT("\t%s: %s\n"), *GuidName, *GuidToString(*Item.puuid));
					break;

				case VT_LPWSTR:
					Dump += FString::Printf(TEXT("\t%s: %s\n"), *GuidName, Item.pwszVal);
					break;

				case VT_VECTOR | VT_UI1:
					Dump += FString::Printf(TEXT("\t%s: <byte array>\n"), *GuidName);
					break;

				case VT_UNKNOWN:
					Dump += FString::Printf(TEXT("\t%s: IUnknown\n"), *GuidName);
					break;

				default:
					Dump += FString::Printf(TEXT("\t%s: Unknown value type %d\n"), *GuidName, Item.vt);
					break;
				}
			}
		}

		return Dump;
	}


	void EnumerateCaptureDevices(GUID DeviceType, TArray<TComPtr<IMFActivate>>& OutDevices)
	{
		if ((DeviceType != MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID) &&
			(DeviceType != MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID))
		{
			return; // unsupported device type
		}

		// create attribute store for search criteria
		TComPtr<IMFAttributes> Attributes;
		{
			const HRESULT Result = ::MFCreateAttributes(&Attributes, 1);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to create capture device enumeration attributes: %ls", *ResultToString(Result));
				return;
			}
		}

		// request capture devices
		{
			const HRESULT Result = Attributes->SetGUID(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE, DeviceType);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to set capture device enumeration type: %ls", *ResultToString(Result));
				return;
			}
		}

		IMFActivate** Devices = nullptr;
		UINT32 DeviceCount = 0;
		{
			const HRESULT Result = ::MFEnumDeviceSources(Attributes, &Devices, &DeviceCount);

			if (DeviceCount == 0)
			{
				return; // no devices found
			}
		}

		for (DWORD DeviceIndex = 0; DeviceIndex < DeviceCount; ++DeviceIndex)
		{
			TComPtr<IMFActivate> Device = Devices[DeviceIndex];
			OutDevices.Add(Device);
			Device->Release();
		}

		::CoTaskMemFree(Devices);
	}


	FString FourccToString(unsigned long Fourcc)
	{
		FString Result;

		for (int32 CharIndex = 0; CharIndex < 4; ++CharIndex)
		{
			const unsigned char C = Fourcc & 0xff;
			if (TChar<char>::IsPrint(C))
			{
				Result += FString::Printf(TEXT("%c"), C);
			}
			else
			{
				Result += FString::Printf(TEXT("[%d]"), C);
			}
			Fourcc >>= 8;
		}

		return Result.Reverse();
	}


	FString FormatTypeToString(GUID FormatType)
	{
		if (FormatType == FORMAT_DvInfo) return TEXT("DVINFO");
		if (FormatType == FORMAT_MPEG2Video) return TEXT("MPEG2VIDEOINFO");
		if (FormatType == FORMAT_MPEGStreams) return TEXT("AM_MPEGSYSTEMTYPE");
		if (FormatType == FORMAT_MPEGVideo) return TEXT("MPEG1VIDEOINFO");
		if (FormatType == FORMAT_None) return TEXT("None");
		if (FormatType == FORMAT_VideoInfo) return TEXT("VIDEOINFOHEADER");
		if (FormatType == FORMAT_VideoInfo2) return TEXT("VIDEOINFOHEADER2");
		if (FormatType == FORMAT_WaveFormatEx) return TEXT("WAVEFORMATEX");
		if (FormatType == FORMAT_525WSS) return TEXT("525WSS");
		if (FormatType == GUID_NULL) return TEXT("Null");

		return FString::Printf(TEXT("Unknown format type %s"), *GuidToString(FormatType));
	}


	bool FrameRateToRatio(float FrameRate, int32& OutNumerator, int32& OutDenominator)
	{
		if (FrameRate < 0.0f)
		{
			return false;
		}

		// use lookup table first to match WMF behavior
		// see https://msdn.microsoft.com/en-us/library/windows/desktop/aa370467(v=vs.85).aspx

		for (int32 LutIndex = 0; LutIndex < WMFMEDIA_FRAMERATELUT_SIZE; ++LutIndex)
		{
			const FFrameRateLut& Lut = FrameRateLut[LutIndex];

			if (Lut.FrameRate == FrameRate)
			{
				OutNumerator = Lut.Numerator;
				OutDenominator = Lut.Denominator;

				return true;
			}
		}

		// calculate a ratio (we could do better here, but this is fast)
		const int32 NumeratorScale = 10000;

		if (FrameRate > (MAX_int32 / NumeratorScale))
		{
			return false;
		}

		OutNumerator = (int32)(FrameRate * NumeratorScale);
		OutDenominator = NumeratorScale;

		return true;
	}


	bool GetCaptureDeviceInfo(IMFActivate& Device, FText& OutDisplayName, FString& OutInfo, bool& OutSoftwareDevice, FString& OutUrl)
	{
		GUID DeviceType;
		{
			if (FAILED(Device.GetGUID(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE, &DeviceType)))
			{
				return false; // failed to get device type
			}

			if ((DeviceType != MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID) &&
				(DeviceType != MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID))
			{
				return false; // unsupported device type
			}
		}

		const bool IsAudioDevice = (DeviceType == MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID);

		// display name
		PWSTR OutString = NULL;
		UINT32 OutLength = 0;

		if (SUCCEEDED(Device.GetAllocatedString(MF_DEVSOURCE_ATTRIBUTE_FRIENDLY_NAME, &OutString, &OutLength)))
		{
			OutDisplayName = FText::FromString(OutString);
			::CoTaskMemFree(OutString);
		}
		else
		{
			OutDisplayName = NSLOCTEXT("WmfMedia", "UnknownCaptureDeviceName", "Unknown");
		}

		// debug information
		if (IsAudioDevice)
		{
			UINT32 Role = 0;
			if (SUCCEEDED(Device.GetUINT32(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_ROLE, &Role)))
			{
				OutInfo += TEXT("Role: ") + CaptureDeviceRoleToString((ERole)Role) + TEXT("\n");
			}
		}
		else
		{
			UINT32 MaxBuffers = 0;
			if (SUCCEEDED(Device.GetUINT32(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_MAX_BUFFERS, &MaxBuffers)))
			{
				OutInfo += FString::Printf(TEXT("Max Buffers: %i"), MaxBuffers);
			}
		}

		// software device
		UINT32 HwSource = 0;

		if (SUCCEEDED(Device.GetUINT32(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_HW_SOURCE, &HwSource)))
		{
			OutSoftwareDevice = (HwSource == 0);
		}
		else
		{
			OutSoftwareDevice = false;
		}

		// symbolic link
		const GUID SymbolicLinkAttribute = IsAudioDevice ? MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_ENDPOINT_ID : MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_SYMBOLIC_LINK;
		const FString UrlScheme = IsAudioDevice ? TEXT("audcap://") : TEXT("vidcap://");

#pragma warning(push)
#pragma warning(disable: 6388) // CA Warning: OutLength may not be NULL - According to MS documentation the initial value does not matter & we are sure to query a string type
		if (SUCCEEDED(Device.GetAllocatedString(SymbolicLinkAttribute, &OutString, &OutLength)))
		{
			OutUrl = UrlScheme + OutString;
			::CoTaskMemFree(OutString);
		}
#pragma warning(pop)

		return true;
	}


	const TArray<TComPtr<IMFMediaType>>& GetSupportedMediaTypes(const GUID& MajorType)
	{
		InitializeSupportedTypes();

		if (MajorType == MFMediaType_Audio)
		{
			return SupportedAudioTypes;
		}

		if (MajorType == MFMediaType_Binary)
		{
			return SupportedBinaryTypes;
		}

		if (MajorType == MFMediaType_SAMI)
		{
			return SupportedSamiTypes;
		}

		if (MajorType == MFMediaType_Video)
		{
			return SupportedVideoTypes;
		}

		static TArray<TComPtr<IMFMediaType>> Empty;

		return Empty;
	}


	HRESULT GetTopologyFromEvent(IMFMediaEvent& Event, TComPtr<IMFTopology>& OutTopology)
	{
		HRESULT Result = S_OK;
		PROPVARIANT Variant;

		PropVariantInit(&Variant);
		Result = Event.GetValue(&Variant);

		if (SUCCEEDED(Result))
		{
			if (Variant.vt != VT_UNKNOWN)
			{
				Result = E_UNEXPECTED;
			}
		}

		if (SUCCEEDED(Result))
		{
			Result = Variant.punkVal->QueryInterface(IID_PPV_ARGS(&OutTopology));
		}

		PropVariantClear(&Variant);

		return Result;
	}


	FString GuidToString(const GUID& Guid)
	{
		return FString::Printf(TEXT("%08x-%04x-%04x-%02x%02x%02x%02x%02x%02x%02x%02x"),
			(uint32)Guid.Data1,
			(uint32)Guid.Data2,
			(uint32)Guid.Data3,
			(uint32)Guid.Data4[0],
			(uint32)Guid.Data4[1],
			(uint32)Guid.Data4[2],
			(uint32)Guid.Data4[3],
			(uint32)Guid.Data4[4],
			(uint32)Guid.Data4[5],
			(uint32)Guid.Data4[6],
			(uint32)Guid.Data4[7]
		);
	}


	FString InterlaceModeToString(MFVideoInterlaceMode Mode)
	{
		switch (Mode)
		{
		case MFVideoInterlace_Unknown: return TEXT("Unknown");
		case MFVideoInterlace_Progressive: return TEXT("Progressive");
		case MFVideoInterlace_FieldInterleavedUpperFirst: return TEXT("Field Interleaved Upper First");
		case MFVideoInterlace_FieldInterleavedLowerFirst: return TEXT("Field Intereaved Lower First");
		case MFVideoInterlace_FieldSingleUpper: return TEXT("Field Single Upper");
		case MFVideoInterlace_FieldSingleLower: return TEXT("Field Single Lower");
		case MFVideoInterlace_MixedInterlaceOrProgressive: return TEXT("Mixed Interlace or Progressive");
		
		default:
			return FString::Printf(TEXT("Unknown mode %i"), (int32)Mode);
		}
	}


	bool IsSupportedMajorType(const GUID& MajorType)
	{
		for (GUID const* const Type : SupportedMajorTypes)
		{
			if (*Type == MajorType)
			{
				return true;
			}
		}

		return false;
	}


	FString MajorTypeToString(const GUID& MajorType)
	{
		if (MajorType == MFMediaType_Default) return TEXT("Default");
		if (MajorType == MFMediaType_Audio) return TEXT("Audio");
		if (MajorType == MFMediaType_Video) return TEXT("Video");
		if (MajorType == MFMediaType_Protected) return TEXT("Protected");
		if (MajorType == MFMediaType_SAMI) return TEXT("SAMI");
		if (MajorType == MFMediaType_Script) return TEXT("Script");
		if (MajorType == MFMediaType_Image) return TEXT("Image");
		if (MajorType == MFMediaType_HTML) return TEXT("HTML");
		if (MajorType == MFMediaType_Binary) return TEXT("Binary");
		if (MajorType == MFMediaType_FileTransfer) return TEXT("FileTransfer"); 
		if (MajorType == MFMediaType_Stream) return TEXT("Stream");

		return GuidToString(MajorType);
	}


	FString MarkerTypeToString(MFSTREAMSINK_MARKER_TYPE MarkerType)
	{
		switch (MarkerType)
		{
		case MFSTREAMSINK_MARKER_DEFAULT: return TEXT("Default");
		case MFSTREAMSINK_MARKER_ENDOFSEGMENT: return TEXT("End Of Segment");
		case MFSTREAMSINK_MARKER_TICK: return TEXT("Tick");
		case MFSTREAMSINK_MARKER_EVENT: return TEXT("Event");

		default:
			return FString::Printf(TEXT("Unknown marker type %i"), (int32)MarkerType);
		}
	}


	FString MediaEventToString(MediaEventType Event)
	{
		switch (Event)
		{
		case MEUnknown: return TEXT("Unknown");
		case MEError: return TEXT("Error");
		case MEExtendedType: return TEXT("Extended Type");
		case MENonFatalError: return TEXT("Non-fatal Error");
		case MESessionUnknown: return TEXT("Session Unknown");
		case MESessionTopologySet: return TEXT("Session Topology Set");
		case MESessionTopologiesCleared: return TEXT("Session Topologies Cleared");
		case MESessionStarted: return TEXT("Session Started");
		case MESessionPaused: return TEXT("Session Paused");
		case MESessionStopped: return TEXT("Session Stopped");
		case MESessionClosed: return TEXT("Session Closed");
		case MESessionEnded: return TEXT("Session Ended");
		case MESessionRateChanged: return TEXT("Session Rate Changed");
		case MESessionScrubSampleComplete: return TEXT("Session Scrub Sample Complete");
		case MESessionCapabilitiesChanged: return TEXT("Session Capabilities Changed");
		case MESessionTopologyStatus: return TEXT("Session Topology Status");
		case MESessionNotifyPresentationTime: return TEXT("Session Notify Presentation Time");
		case MENewPresentation: return TEXT("New Presentation");
		case MELicenseAcquisitionStart: return TEXT("License Acquisition Start");
		case MELicenseAcquisitionCompleted: return TEXT("License Acquisition Completed");
		case MEIndividualizationStart: return TEXT("Individualization Start");
		case MEIndividualizationCompleted: return TEXT("Individualization Completed");
		case MEEnablerProgress: return TEXT("Enabler Progress");
		case MEEnablerCompleted: return TEXT("Enabler Completed");
		case MEPolicyError: return TEXT("Policy Error");
		case MEPolicyReport: return TEXT("Policy Report");
		case MEBufferingStarted: return TEXT("Buffering Started");
		case MEBufferingStopped: return TEXT("Buffering Stopped");
		case MEConnectStart: return TEXT("Connect Start");
		case MEConnectEnd: return TEXT("Connect End");
		case MEReconnectStart: return TEXT("Reconnect Start");
		case MEReconnectEnd: return TEXT("Reconnect End");
		case MERendererEvent: return TEXT("Renderer Event");
		case MESessionStreamSinkFormatChanged: return TEXT("Session Stream Sink Format Changed");
		case MESourceUnknown: return TEXT("Source Unknown");
		case MESourceStarted: return TEXT("Source Started");
		case MEStreamStarted: return TEXT("Stream Started");
		case MESourceSeeked: return TEXT("Source Seeked");
		case MEStreamSeeked: return TEXT("Stream Seeked");
		case MENewStream: return TEXT("New Stream");
		case MEUpdatedStream: return TEXT("Updated Stream");
		case MESourceStopped: return TEXT("Source Stopped");
		case MEStreamStopped: return TEXT("Stream Stopped");
		case MESourcePaused: return TEXT("Source Paused");
		case MEStreamPaused: return TEXT("Stream Paused");
		case MEEndOfPresentation: return TEXT("End of Presentation");
		case MEEndOfStream: return TEXT("End of Stream");
		case MEMediaSample: return TEXT("Media Sample");
		case MEStreamTick: return TEXT("Stream Tick");
		case MEStreamThinMode: return TEXT("Stream Thin Mode");
		case MEStreamFormatChanged: return TEXT("Stream Format Changed");
		case MESourceRateChanged: return TEXT("Source Rate Changed");
		case MEEndOfPresentationSegment: return TEXT("End of Presentation Segment");
		case MESourceCharacteristicsChanged: return TEXT("Source Characteristics Changed");
		case MESourceRateChangeRequested: return TEXT("Source Rate Change Requested");
		case MESourceMetadataChanged: return TEXT("Source Metadata Changed");
		case MESequencerSourceTopologyUpdated: return TEXT("Sequencer Source Topology Updated");
		case MESinkUnknown: return TEXT("Sink Unknown");
		case MEStreamSinkStarted: return TEXT("Stream Sink Started");
		case MEStreamSinkStopped: return TEXT("Stream Sink Stopped");
		case MEStreamSinkPaused: return TEXT("Strema Sink Paused");
		case MEStreamSinkRateChanged: return TEXT("Stream Sink Rate Changed");
		case MEStreamSinkRequestSample: return TEXT("Stream Sink Request Sample");
		case MEStreamSinkMarker: return TEXT("Stream Sink Marker");
		case MEStreamSinkPrerolled: return TEXT("Stream Sink Prerolled");
		case MEStreamSinkScrubSampleComplete: return TEXT("Stream Sink Scrub Sample Complete");
		case MEStreamSinkFormatChanged: return TEXT("Stream Sink Format Changed");
		case MEStreamSinkDeviceChanged: return TEXT("Stream Sink Device Changed");
		case MEQualityNotify: return TEXT("Quality Notify");
		case MESinkInvalidated: return TEXT("Sink Invalidated");
		case MEAudioSessionNameChanged: return TEXT("Audio Session Name Changed");
		case MEAudioSessionVolumeChanged: return TEXT("Audio Session Volume Changed");
		case MEAudioSessionDeviceRemoved: return TEXT("Audio Session Device Removed");
		case MEAudioSessionServerShutdown: return TEXT("Audio Session Server Shutdown");
		case MEAudioSessionGroupingParamChanged: return TEXT("Audio Session Grouping Param Changed");
		case MEAudioSessionIconChanged: return TEXT("Audio Session Icion Changed");
		case MEAudioSessionFormatChanged: return TEXT("Audio Session Format Changed");
		case MEAudioSessionDisconnected: return TEXT("Audio Session Disconnected");
		case MEAudioSessionExclusiveModeOverride: return TEXT("Audio Session Exclusive Mode Override");
		case MECaptureAudioSessionVolumeChanged: return TEXT("Capture Audio Session Volume Changed");
		case MECaptureAudioSessionDeviceRemoved: return TEXT("Capture Audio Session Device Removed");
		case MECaptureAudioSessionFormatChanged: return TEXT("Capture Audio Session Format Changed");
		case MECaptureAudioSessionDisconnected: return TEXT("Capture Audio Session Disconnected");
		case MECaptureAudioSessionExclusiveModeOverride: return TEXT("Capture Audio Session Exclusive Mode Override");
		case MECaptureAudioSessionServerShutdown: return TEXT("Capture Audio Session Server Shutdown");
		case METrustUnknown: return TEXT("Trust Unknown");
		case MEPolicyChanged: return TEXT("Policy Changed");
		case MEContentProtectionMessage: return TEXT("Content Protection Message");
		case MEPolicySet: return TEXT("Policy Set");
		case MEWMDRMLicenseBackupCompleted: return TEXT("WM DRM License Backup Completed");
		case MEWMDRMLicenseBackupProgress: return TEXT("WM DRM License Backup Progress");
		case MEWMDRMLicenseRestoreCompleted: return TEXT("WM DRM License Restore Completed");
		case MEWMDRMLicenseRestoreProgress: return TEXT("WM DRM License Restore Progress");
		case MEWMDRMLicenseAcquisitionCompleted: return TEXT("WM DRM License Acquisition Completed");
		case MEWMDRMIndividualizationCompleted: return TEXT("WM DRM Individualization Completed");
		case MEWMDRMIndividualizationProgress: return TEXT("WM DRM Individualization Progress");
		case MEWMDRMProximityCompleted: return TEXT("WM DRM Proximity Completed");
		case MEWMDRMLicenseStoreCleaned: return TEXT("WM DRM License Store Cleaned");
		case MEWMDRMRevocationDownloadCompleted: return TEXT("WM DRM Revocation Download Completed");
		case METransformUnknown: return TEXT("Transform Unkonwn");
		case METransformNeedInput: return TEXT("Transform Need Input");
		case METransformHaveOutput: return TEXT("Transform Have Output");
		case METransformDrainComplete: return TEXT("Transform Drain Complete");
		case METransformMarker: return TEXT("Transform Marker");
		case MEByteStreamCharacteristicsChanged: return TEXT("Byte Stream Characteristics Changed");
		case MEVideoCaptureDeviceRemoved: return TEXT("Video Capture Device Removed");
		case MEVideoCaptureDevicePreempted: return TEXT("Video Capture Device Preempted");
		case MEStreamSinkFormatInvalidated: return TEXT("Stream Sink Format Invalidated");
		case MEEncodingParameters: return TEXT("Encoding Paramters");
		case MEContentProtectionMetadata: return TEXT("Content Protection Metadata");

		default:
			return FString::Printf(TEXT("Unknown event %i"), (int32)Event);
		}
	}


	float RatioToFrameRate(int32 Numerator, int32 Denominator)
	{
		if (Denominator == 0)
		{
			return false;
		}

		// use lookup table first to match WMF behavior
		// see https://msdn.microsoft.com/en-us/library/windows/desktop/aa370467(v=vs.85).aspx

		for (int32 LutIndex = 0; LutIndex < WMFMEDIA_FRAMERATELUT_SIZE; ++LutIndex)
		{
			const FFrameRateLut& Lut = FrameRateLut[LutIndex];

			if ((Lut.Numerator == Numerator) && (Lut.Denominator == Denominator))
			{
				return Lut.FrameRate;
			}
		}

		return ((float)Numerator / (float)Denominator);
	}


	TComPtr<IMFMediaSource> ResolveMediaSource(TSharedPtr<FArchive, ESPMode::ThreadSafe> Archive, const FString& Url, bool Precache)
	{
		if (!Archive.IsValid())
		{
			const bool IsAudioDevice = Url.StartsWith(TEXT("audcap://"));
			
			// create capture device media source
			if (IsAudioDevice || Url.StartsWith(TEXT("vidcap://")))
			{
				const TCHAR* EndpointOrSymlink = &Url[9];

				TComPtr<IMFAttributes> Attributes;
				{
					HRESULT Result = ::MFCreateAttributes(&Attributes, 2);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Error, "Failed to create capture device attributes: %ls", *ResultToString(Result));
						return NULL;
					}

					Result = Attributes->SetGUID(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE, IsAudioDevice ? MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID : MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Error, "Failed to set capture device source type attribute: %ls", *ResultToString(Result));
						return NULL;
					}

					Result = Attributes->SetString(IsAudioDevice ? MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_ENDPOINT_ID : MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_SYMBOLIC_LINK, EndpointOrSymlink);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Error, "Failed to set capture device endpoint/symlink attribute: %ls", *ResultToString(Result));
						return NULL;
					}
				}

				TComPtr<IMFMediaSource> MediaSource;
				{
					const HRESULT Result = ::MFCreateDeviceSource(Attributes, &MediaSource);

					if (FAILED(Result))
					{
						UE_LOGF(LogWmfMedia, Error, "Failed to create capture device media source: %ls", *ResultToString(Result));
						return NULL;
					}
				}

				return MediaSource;
			}

			// load file media source
			if (Url.StartsWith(TEXT("file://")))
			{
				const TCHAR* FilePath = &Url[7];

				if (Precache)
				{
					FArrayReader* Reader = new FArrayReader;

					if (FFileHelper::LoadFileToArray(*Reader, FilePath))
					{
						Archive = MakeShareable(Reader);
					}
					else
					{
						delete Reader;
					}
				}
				else
				{
					Archive = MakeShareable(IFileManager::Get().CreateFileReader(FilePath));
				}

				if (!Archive.IsValid())
				{
					UE_LOGF(LogWmfMedia, Error, "Failed to open or read media file %ls", FilePath);
					return NULL;
				}

				if (Archive->TotalSize() == 0)
				{
					UE_LOGF(LogWmfMedia, Error, "Cannot open media from empty file %ls.", FilePath);
					return NULL;
				}
			}
		}

		// create source resolver
		TComPtr<IMFSourceResolver> SourceResolver;
		{
			const HRESULT Result = ::MFCreateSourceResolver(&SourceResolver);

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to create media source resolver: %ls", *ResultToString(Result));
				return NULL;
			}
		}

		// resolve media source
		TComPtr<IUnknown> SourceObject;
		{
			MF_OBJECT_TYPE ObjectType;

			if (Archive.IsValid())
			{
				TComPtr<FWmfMediaByteStream> ByteStream = new FWmfMediaByteStream(Archive.ToSharedRef());
				const HRESULT Result = SourceResolver->CreateObjectFromByteStream(ByteStream, *Url, MF_RESOLUTION_MEDIASOURCE, NULL, &ObjectType, &SourceObject);

				if (FAILED(Result))
				{
					UE_LOGF(LogWmfMedia, Error, "Failed to resolve byte stream %ls: %ls", *Url, *ResultToString(Result));
					return NULL;
				}
			}
			else
			{
				const HRESULT Result = SourceResolver->CreateObjectFromURL(*Url, MF_RESOLUTION_MEDIASOURCE, NULL, &ObjectType, &SourceObject);

				if (FAILED(Result))
				{
					UE_LOGF(LogWmfMedia, Error, "Failed to resolve URL %ls: %ls", *Url, *ResultToString(Result));
					return NULL;
				}
			}
		}

		// get media source interface
		TComPtr<IMFMediaSource> MediaSource;
		{
			const HRESULT Result = SourceObject->QueryInterface(IID_PPV_ARGS(&MediaSource));

			if (FAILED(Result))
			{
				UE_LOGF(LogWmfMedia, Error, "Failed to query media source interface: %ls", *ResultToString(Result));
				return NULL;
			}
		}

		return MediaSource;
	}


	FString ResultToString(HRESULT Result)
	{
		void* DllHandle = nullptr;

		// load error resource library
		if (HRESULT_FACILITY(Result) == FACILITY_MF)
		{
			const LONG Code = HRESULT_CODE(Result);

			if (((Code >= 0) && (Code <= 1199)) || ((Code >= 3000) && (Code <= 13999)))
			{
				static void* WmErrorDll = nullptr;

				if (WmErrorDll == nullptr)
				{
					WmErrorDll = FPlatformProcess::GetDllHandle(TEXT("wmerror.dll"));
				}

				DllHandle = WmErrorDll;
			}
			else if ((Code >= 2000) && (Code <= 2999))
			{
				static void* AsfErrorDll = nullptr;

				if (AsfErrorDll == nullptr)
				{
					AsfErrorDll = FPlatformProcess::GetDllHandle(TEXT("asferror.dll"));
				}

				DllHandle = AsfErrorDll;
			}
			else if ((Code >= 14000) & (Code <= 44999))
			{
				static void* MfErrorDll = nullptr;

				if (MfErrorDll == nullptr)
				{
					MfErrorDll = FPlatformProcess::GetDllHandle(TEXT("mferror.dll"));
				}

				DllHandle = MfErrorDll;
			}
		}

		TCHAR Buffer[1024];
		Buffer[0] = TEXT('\0');
		DWORD BufferLength = 0;

		// resolve error code
		if (DllHandle != nullptr)
		{
			BufferLength = FormatMessage(FORMAT_MESSAGE_FROM_HMODULE | FORMAT_MESSAGE_IGNORE_INSERTS, DllHandle, Result, MAKELANGID(LANG_NEUTRAL, SUBLANG_DEFAULT), Buffer, 1024, NULL);
		}
		else
		{
			BufferLength = FormatMessage(FORMAT_MESSAGE_FROM_SYSTEM | FORMAT_MESSAGE_IGNORE_INSERTS, NULL, Result, MAKELANGID(LANG_NEUTRAL, SUBLANG_DEFAULT), Buffer, 1024, NULL);
		}

		if (BufferLength == 0)
		{
			return FString::Printf(TEXT("0x%08x"), (uint32)Result);
		}

		// remove line break
		TCHAR* NewLine = FCString::Strchr(Buffer, TEXT('\r'));

		if (NewLine != nullptr)
		{
			*NewLine = TEXT('\0');
		}

		return Buffer;
	}


	FString SubTypeToString(const GUID& SubType)
	{
		if (SubType == GUID_NULL) return TEXT("Null");

		// image formats
		if (SubType == MFImageFormat_JPEG) return TEXT("Jpeg");
		if (SubType == MFImageFormat_RGB32) return TEXT("RGB32");

		// stream formats
		if (SubType == MFStreamFormat_MPEG2Transport) return TEXT("MPEG-2 Transport");
		if (SubType == MFStreamFormat_MPEG2Program) return TEXT("MPEG-2 Program");

		// video formats
		if (SubType == MFVideoFormat_RGB32) return TEXT("RGB32");
		if (SubType == MFVideoFormat_ARGB32) return TEXT("ARGB32");
		if (SubType == MFVideoFormat_RGB24) return TEXT("RGB24");
		if (SubType == MFVideoFormat_RGB555) return TEXT("RGB525");
		if (SubType == MFVideoFormat_RGB565) return TEXT("RGB565");
		if (SubType == MFVideoFormat_RGB8) return TEXT("RGB8");
		if (SubType == MFVideoFormat_AI44) return TEXT("AI44");
		if (SubType == MFVideoFormat_AYUV) return TEXT("AYUV");
		if (SubType == MFVideoFormat_YUY2) return TEXT("YUY2");
		if (SubType == MFVideoFormat_YVYU) return TEXT("YVYU");
		if (SubType == MFVideoFormat_YVU9) return TEXT("YVU9");
		if (SubType == MFVideoFormat_UYVY) return TEXT("UYVY");
		if (SubType == MFVideoFormat_NV11) return TEXT("NV11");
		if (SubType == MFVideoFormat_NV12) return TEXT("NV12");
		if (SubType == MFVideoFormat_YV12) return TEXT("YV12");
		if (SubType == MFVideoFormat_I420) return TEXT("I420");
		if (SubType == MFVideoFormat_IYUV) return TEXT("IYUV");
		if (SubType == MFVideoFormat_Y210) return TEXT("Y210");
		if (SubType == MFVideoFormat_Y216) return TEXT("Y216");
		if (SubType == MFVideoFormat_Y410) return TEXT("Y410");
		if (SubType == MFVideoFormat_Y416) return TEXT("Y416");
		if (SubType == MFVideoFormat_Y41P) return TEXT("Y41P");
		if (SubType == MFVideoFormat_Y41T) return TEXT("Y41T");
		if (SubType == MFVideoFormat_Y42T) return TEXT("Y42T");
		if (SubType == MFVideoFormat_P210) return TEXT("P210");
		if (SubType == MFVideoFormat_P216) return TEXT("P216");
		if (SubType == MFVideoFormat_P010) return TEXT("P010");
		if (SubType == MFVideoFormat_P016) return TEXT("P016");
		if (SubType == MFVideoFormat_v210) return TEXT("v210");
		if (SubType == MFVideoFormat_v216) return TEXT("v216");
		if (SubType == MFVideoFormat_v410) return TEXT("v410");
		if (SubType == MFVideoFormat_MP43) return TEXT("MP43");
		if (SubType == MFVideoFormat_MP4S) return TEXT("MP4S");
		if (SubType == MFVideoFormat_M4S2) return TEXT("M4S2");
		if (SubType == MFVideoFormat_MP4V) return TEXT("MP4V");
		if (SubType == MFVideoFormat_WMV1) return TEXT("WMV1");
		if (SubType == MFVideoFormat_WMV2) return TEXT("WMV2");
		if (SubType == MFVideoFormat_WMV3) return TEXT("WMV3");
		if (SubType == MFVideoFormat_WVC1) return TEXT("WVC1");
		if (SubType == MFVideoFormat_MSS1) return TEXT("MSS1");
		if (SubType == MFVideoFormat_MSS2) return TEXT("MSS2");
		if (SubType == MFVideoFormat_MPG1) return TEXT("MPG1");
		if (SubType == MFVideoFormat_DVSL) return TEXT("DVSL");
		if (SubType == MFVideoFormat_DVSD) return TEXT("DVSD");
		if (SubType == MFVideoFormat_DVHD) return TEXT("DVHD");
		if (SubType == MFVideoFormat_DV25) return TEXT("DV25");
		if (SubType == MFVideoFormat_DV50) return TEXT("DV50");
		if (SubType == MFVideoFormat_DVH1) return TEXT("DVH1");
		if (SubType == MFVideoFormat_DVC) return TEXT("DVC");
		if (SubType == MFVideoFormat_H264) return TEXT("H264");
		if (SubType == MFVideoFormat_MJPG) return TEXT("MJPG");
		if (SubType == MFVideoFormat_420O) return TEXT("420O");
		if (SubType == MFVideoFormat_HEVC) return TEXT("HEVC");
		if (SubType == MFVideoFormat_HEVC_ES) return TEXT("HEVC ES");

#if (WINVER >= _WIN32_WINNT_WIN8)
		if (SubType == MFVideoFormat_H263) return TEXT("H263");
#endif

		if (SubType == MFVideoFormat_H264_ES) return TEXT("H264 ES");
		if (SubType == MFVideoFormat_MPEG2) return TEXT("MPEG-2");

		// common non-Windows formats
		if (SubType == OtherVideoFormat_LifeCam) return TEXT("LifeCam");
		if (SubType == OtherVideoFormat_QuickTime) return TEXT("QuickTime");

		if (IsFormatFamily(SubType, OtherFormatMpeg2_Base))
		{
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_AC3) return TEXT("MPEG-2 AC3");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_AUDIO) return TEXT("MPEG-2 Audio");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_DOLBY_AC3) return TEXT("Dolby AC-3");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_DTS) return TEXT("DTS");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_LPCM_AUDIO) return TEXT("DVD LPCM");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_SDDS) return TEXT("SDDS");

			if (SubType.Data1 == OTHER_FORMAT_MPEG2_DVD_SUBPICTURE) return TEXT("DVD Subpicture");
			if (SubType.Data1 == OTHER_FORMAT_MPEG2_VIDEO) return TEXT("MPEG-2 Video");
		}

		// audio formats
		if (IsFormatFamily(SubType, MFAudioFormat_Base) ||
			IsFormatFamily(SubType, MFMPEG4Format_Base))
		{
			if (SubType.Data1 == WAVE_FORMAT_UNKNOWN) return TEXT("Unknown Audio Format");
			if (SubType.Data1 == WAVE_FORMAT_PCM) return TEXT("PCM");
			if (SubType.Data1 == WAVE_FORMAT_ADPCM) return TEXT("ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_IEEE_FLOAT) return TEXT("IEEE Float");
			if (SubType.Data1 == WAVE_FORMAT_VSELP) return TEXT("VSELP");
			if (SubType.Data1 == WAVE_FORMAT_IBM_CVSD) return TEXT("IBM CVSD");
			if (SubType.Data1 == WAVE_FORMAT_ALAW) return TEXT("aLaw");
			if (SubType.Data1 == WAVE_FORMAT_MULAW) return TEXT("uLaw");
			if (SubType.Data1 == WAVE_FORMAT_DTS) return TEXT("DTS");
			if (SubType.Data1 == WAVE_FORMAT_DRM) return TEXT("DRM");
			if (SubType.Data1 == WAVE_FORMAT_WMAVOICE9) return TEXT("WMA Voice 9");
			if (SubType.Data1 == WAVE_FORMAT_WMAVOICE10) return TEXT("WMA Voice 10");
			if (SubType.Data1 == WAVE_FORMAT_OKI_ADPCM) return TEXT("OKI ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_DVI_ADPCM) return TEXT("Intel DVI ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_IMA_ADPCM) return TEXT("Intel IMA ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_MEDIASPACE_ADPCM) return TEXT("Videologic ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_SIERRA_ADPCM) return TEXT("Sierra ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_G723_ADPCM) return TEXT("G723 ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_DIGISTD) return TEXT("DIGISTD");
			if (SubType.Data1 == WAVE_FORMAT_DIGIFIX) return TEXT("DIGIFIX");
			if (SubType.Data1 == WAVE_FORMAT_DIALOGIC_OKI_ADPCM) return TEXT("Dialogic ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_MEDIAVISION_ADPCM) return TEXT("Media Vision ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_CU_CODEC) return TEXT("HP CU Codec");
			if (SubType.Data1 == WAVE_FORMAT_HP_DYN_VOICE) return TEXT("HP DynVoice");
			if (SubType.Data1 == WAVE_FORMAT_YAMAHA_ADPCM) return TEXT("Yamaha ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_SONARC) return TEXT("Sonarc");
			if (SubType.Data1 == WAVE_FORMAT_DSPGROUP_TRUESPEECH) return TEXT("DPS Group TrueSpeech");
			if (SubType.Data1 == WAVE_FORMAT_ECHOSC1) return TEXT("Echo Speech 1");
			if (SubType.Data1 == WAVE_FORMAT_AUDIOFILE_AF36) return TEXT("AF36");
			if (SubType.Data1 == WAVE_FORMAT_APTX) return TEXT("APTX");
			if (SubType.Data1 == WAVE_FORMAT_AUDIOFILE_AF10) return TEXT("AF10");
			if (SubType.Data1 == WAVE_FORMAT_PROSODY_1612) return TEXT("Prosody 1622");
			if (SubType.Data1 == WAVE_FORMAT_LRC) return TEXT("LRC");
			if (SubType.Data1 == WAVE_FORMAT_DOLBY_AC2) return TEXT("Dolby AC2");
			if (SubType.Data1 == WAVE_FORMAT_GSM610) return TEXT("GSM 610");
			if (SubType.Data1 == WAVE_FORMAT_MSNAUDIO) return TEXT("MSN Audio");
			if (SubType.Data1 == WAVE_FORMAT_ANTEX_ADPCME) return TEXT("Antex ADPCME");
			if (SubType.Data1 == WAVE_FORMAT_CONTROL_RES_VQLPC) return TEXT("Control Resources VQLPC");
			if (SubType.Data1 == WAVE_FORMAT_DIGIREAL) return TEXT("DigiReal");
			if (SubType.Data1 == WAVE_FORMAT_DIGIADPCM) return TEXT("DigiADPCM");
			if (SubType.Data1 == WAVE_FORMAT_CONTROL_RES_CR10) return TEXT("Control Resources CR10");
			if (SubType.Data1 == WAVE_FORMAT_NMS_VBXADPCM) return TEXT("VBX ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_CS_IMAADPCM) return TEXT("Crystal IMA ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_ECHOSC3) return TEXT("Echo Speech 3");
			if (SubType.Data1 == WAVE_FORMAT_ROCKWELL_ADPCM) return TEXT("Rockwell ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_ROCKWELL_DIGITALK) return TEXT("Rockwell DigiTalk");
			if (SubType.Data1 == WAVE_FORMAT_XEBEC) return TEXT("Xebec");
			if (SubType.Data1 == WAVE_FORMAT_G721_ADPCM) return TEXT("G721 ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_G728_CELP) return TEXT("G728 CELP");
			if (SubType.Data1 == WAVE_FORMAT_MSG723) return TEXT("MSG723");
			if (SubType.Data1 == WAVE_FORMAT_INTEL_G723_1) return TEXT("Intel G723.1");
			if (SubType.Data1 == WAVE_FORMAT_INTEL_G729) return TEXT("Intel G729");
			if (SubType.Data1 == WAVE_FORMAT_SHARP_G726) return TEXT("Sharp G726");
			if (SubType.Data1 == WAVE_FORMAT_MPEG) return TEXT("MPEG");
			if (SubType.Data1 == WAVE_FORMAT_RT24) return TEXT("InSoft RT24");
			if (SubType.Data1 == WAVE_FORMAT_PAC) return TEXT("InSoft PAC");
			if (SubType.Data1 == WAVE_FORMAT_MPEGLAYER3) return TEXT("MPEG Layer 3");
			if (SubType.Data1 == WAVE_FORMAT_LUCENT_G723) return TEXT("Lucent G723");
			if (SubType.Data1 == WAVE_FORMAT_CIRRUS) return TEXT("Cirrus Logic");
			if (SubType.Data1 == WAVE_FORMAT_ESPCM) return TEXT("ESS PCM");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE) return TEXT("Voxware");
			if (SubType.Data1 == WAVE_FORMAT_CANOPUS_ATRAC) return TEXT("Canopus ATRAC");
			if (SubType.Data1 == WAVE_FORMAT_G726_ADPCM) return TEXT("APICOM G726");
			if (SubType.Data1 == WAVE_FORMAT_G722_ADPCM) return TEXT("APICOM G722");
			if (SubType.Data1 == WAVE_FORMAT_DSAT) return TEXT("DSAT");
			if (SubType.Data1 == WAVE_FORMAT_DSAT_DISPLAY) return TEXT("DSAT Display");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_BYTE_ALIGNED) return TEXT("Voxware Byte Aligned");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_AC8) return TEXT("Voxware AC8");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_AC10) return TEXT("Voxware AC10");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_AC16) return TEXT("Voxware AC16");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_AC20) return TEXT("Voxware AC20");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_RT24) return TEXT("Voxware RT24");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_RT29) return TEXT("Voxware RT29");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_RT29HW) return TEXT("Voxware RT29HW");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_VR12) return TEXT("Voxware VR12");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_VR18) return TEXT("Voxware VR18");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_TQ40) return TEXT("Voxware TQ40");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_SC3) return TEXT("Voxware SC3");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_SC3_1) return TEXT("Voxware SC3.1");
			if (SubType.Data1 == WAVE_FORMAT_SOFTSOUND) return TEXT("Softsound");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_TQ60) return TEXT("Voxware TQ60");
			if (SubType.Data1 == WAVE_FORMAT_MSRT24) return TEXT("MSRT24");
			if (SubType.Data1 == WAVE_FORMAT_G729A) return TEXT("AT&T G729A");
			if (SubType.Data1 == WAVE_FORMAT_MVI_MVI2) return TEXT("NVI2");
			if (SubType.Data1 == WAVE_FORMAT_DF_G726) return TEXT("DataFusion G726");
			if (SubType.Data1 == WAVE_FORMAT_DF_GSM610) return TEXT("DataFusion GSM610");
			if (SubType.Data1 == WAVE_FORMAT_ISIAUDIO) return TEXT("Iterated Systems");
			if (SubType.Data1 == WAVE_FORMAT_ONLIVE) return TEXT("OnLive!");
			if (SubType.Data1 == WAVE_FORMAT_MULTITUDE_FT_SX20) return TEXT("Multitude FT SX20");
			if (SubType.Data1 == WAVE_FORMAT_INFOCOM_ITS_G721_ADPCM) return TEXT("Infocom ITS G721 ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_CONVEDIA_G729) return TEXT("Convedia G729");
			if (SubType.Data1 == WAVE_FORMAT_CONGRUENCY) return TEXT("Congruency");
			if (SubType.Data1 == WAVE_FORMAT_SBC24) return TEXT("SBC24");
			if (SubType.Data1 == WAVE_FORMAT_DOLBY_AC3_SPDIF) return TEXT("Dolby AC3 SPDIF");
			if (SubType.Data1 == WAVE_FORMAT_MEDIASONIC_G723) return TEXT("MediaSonic G723");
			if (SubType.Data1 == WAVE_FORMAT_PROSODY_8KBPS) return TEXT("Prosody 8kps");
			if (SubType.Data1 == WAVE_FORMAT_ZYXEL_ADPCM) return TEXT("ZyXEL ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_PHILIPS_LPCBB) return TEXT("Philips LPCBB");
			if (SubType.Data1 == WAVE_FORMAT_PACKED) return TEXT("Studer Packed");
			if (SubType.Data1 == WAVE_FORMAT_MALDEN_PHONYTALK) return TEXT("Malden PhonyTalk");
			if (SubType.Data1 == WAVE_FORMAT_RACAL_RECORDER_GSM) return TEXT("Racal GSM");
			if (SubType.Data1 == WAVE_FORMAT_RACAL_RECORDER_G720_A) return TEXT("Racal G720.A");
			if (SubType.Data1 == WAVE_FORMAT_RACAL_RECORDER_G723_1) return TEXT("Racal G723.1");
			if (SubType.Data1 == WAVE_FORMAT_RACAL_RECORDER_TETRA_ACELP) return TEXT("Racal Tetra ACELP");
			if (SubType.Data1 == WAVE_FORMAT_NEC_AAC) return TEXT("NEC AAC");
			if (SubType.Data1 == WAVE_FORMAT_RAW_AAC1) return TEXT("Raw AAC-1");
			if (SubType.Data1 == WAVE_FORMAT_RHETOREX_ADPCM) return TEXT("Rhetorex ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_IRAT) return TEXT("BeCubed IRAT");
			if (SubType.Data1 == WAVE_FORMAT_VIVO_G723) return TEXT("Vivo G723");
			if (SubType.Data1 == WAVE_FORMAT_VIVO_SIREN) return TEXT("vivo Siren");
			if (SubType.Data1 == WAVE_FORMAT_PHILIPS_CELP) return TEXT("Philips Celp");
			if (SubType.Data1 == WAVE_FORMAT_PHILIPS_GRUNDIG) return TEXT("Philips Grundig");
			if (SubType.Data1 == WAVE_FORMAT_DIGITAL_G723) return TEXT("DEC G723");
			if (SubType.Data1 == WAVE_FORMAT_SANYO_LD_ADPCM) return TEXT("Sanyo ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_ACEPLNET) return TEXT("Sipro Lab ACEPLNET");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_ACELP4800) return TEXT("Sipro Lab ACELP4800");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_ACELP8V3) return TEXT("Sipro Lab ACELP8v3");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_G729) return TEXT("Spiro Lab G729");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_G729A) return TEXT("Spiro Lab G729A");
			if (SubType.Data1 == WAVE_FORMAT_SIPROLAB_KELVIN) return TEXT("Spiro Lab Kelvin");
			if (SubType.Data1 == WAVE_FORMAT_VOICEAGE_AMR) return TEXT("VoiceAge AMR");
			if (SubType.Data1 == WAVE_FORMAT_G726ADPCM) return TEXT("Dictaphone G726 ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_DICTAPHONE_CELP68) return TEXT("Dictaphone CELP68");
			if (SubType.Data1 == WAVE_FORMAT_DICTAPHONE_CELP54) return TEXT("Dictaphone CELP54");
			if (SubType.Data1 == WAVE_FORMAT_QUALCOMM_PUREVOICE) return TEXT("Qualcomm PureVoice");
			if (SubType.Data1 == WAVE_FORMAT_QUALCOMM_HALFRATE) return TEXT("Qualcomm Half-Rate");
			if (SubType.Data1 == WAVE_FORMAT_TUBGSM) return TEXT("Ring Zero Systems TUBGSM");
			if (SubType.Data1 == WAVE_FORMAT_MSAUDIO1) return TEXT("Microsoft Audio 1");
			if (SubType.Data1 == WAVE_FORMAT_WMAUDIO2) return TEXT("Windows Media Audio 2");
			if (SubType.Data1 == WAVE_FORMAT_WMAUDIO3) return TEXT("Windows Media Audio 3");
			if (SubType.Data1 == WAVE_FORMAT_WMAUDIO_LOSSLESS) return TEXT("Window Media Audio Lossless");
			if (SubType.Data1 == WAVE_FORMAT_WMASPDIF) return TEXT("Windows Media Audio SPDIF");
			if (SubType.Data1 == WAVE_FORMAT_UNISYS_NAP_ADPCM) return TEXT("Unisys ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_UNISYS_NAP_ULAW) return TEXT("Unisys uLaw");
			if (SubType.Data1 == WAVE_FORMAT_UNISYS_NAP_ALAW) return TEXT("Unisys aLaw");
			if (SubType.Data1 == WAVE_FORMAT_UNISYS_NAP_16K) return TEXT("Unisys 16k");
			if (SubType.Data1 == WAVE_FORMAT_SYCOM_ACM_SYC008) return TEXT("SyCom ACM SYC008");
			if (SubType.Data1 == WAVE_FORMAT_SYCOM_ACM_SYC701_G726L) return TEXT("SyCom ACM SYC701 G726L");
			if (SubType.Data1 == WAVE_FORMAT_SYCOM_ACM_SYC701_CELP54) return TEXT("SyCom ACM SYC701 CELP54");
			if (SubType.Data1 == WAVE_FORMAT_SYCOM_ACM_SYC701_CELP68) return TEXT("SyCom ACM SYC701 CELP68");
			if (SubType.Data1 == WAVE_FORMAT_KNOWLEDGE_ADVENTURE_ADPCM) return TEXT("Knowledge Adventure ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_FRAUNHOFER_IIS_MPEG2_AAC) return TEXT("Fraunhofer MPEG-2 AAC");
			if (SubType.Data1 == WAVE_FORMAT_DTS_DS) return TEXT("DTS DS");
			if (SubType.Data1 == WAVE_FORMAT_CREATIVE_ADPCM) return TEXT("Creative Labs ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_CREATIVE_FASTSPEECH8) return TEXT("Creative Labs FastSpeech 8");
			if (SubType.Data1 == WAVE_FORMAT_CREATIVE_FASTSPEECH10) return TEXT("Creative Labs FastSpeech 10");
			if (SubType.Data1 == WAVE_FORMAT_UHER_ADPCM) return TEXT("UHER ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_ULEAD_DV_AUDIO) return TEXT("Ulead DV Audio");
			if (SubType.Data1 == WAVE_FORMAT_ULEAD_DV_AUDIO_1) return TEXT("Ulead DV Audio.1");
			if (SubType.Data1 == WAVE_FORMAT_QUARTERDECK) return TEXT("Quarterdeck");
			if (SubType.Data1 == WAVE_FORMAT_ILINK_VC) return TEXT("I-link VC");
			if (SubType.Data1 == WAVE_FORMAT_RAW_SPORT) return TEXT("RAW SPORT");
			if (SubType.Data1 == WAVE_FORMAT_ESST_AC3) return TEXT("ESS Technology AC3");
			if (SubType.Data1 == WAVE_FORMAT_GENERIC_PASSTHRU) return TEXT("Generic Passthrough");
			if (SubType.Data1 == WAVE_FORMAT_IPI_HSX) return TEXT("IPI HSX");
			if (SubType.Data1 == WAVE_FORMAT_IPI_RPELP) return TEXT("IPI RPELP");
			if (SubType.Data1 == WAVE_FORMAT_CS2) return TEXT("Consistent Software 2");
			if (SubType.Data1 == WAVE_FORMAT_SONY_SCX) return TEXT("Sony SCX");
			if (SubType.Data1 == WAVE_FORMAT_SONY_SCY) return TEXT("Sony SCY");
			if (SubType.Data1 == WAVE_FORMAT_SONY_ATRAC3) return TEXT("Sony ATRAC3");
			if (SubType.Data1 == WAVE_FORMAT_SONY_SPC) return TEXT("Sony SPC");
			if (SubType.Data1 == WAVE_FORMAT_TELUM_AUDIO) return TEXT("Telum Audio");
			if (SubType.Data1 == WAVE_FORMAT_TELUM_IA_AUDIO) return TEXT("Telum IA Audio");
			if (SubType.Data1 == WAVE_FORMAT_NORCOM_VOICE_SYSTEMS_ADPCM) return TEXT("Norcom ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_FM_TOWNS_SND) return TEXT("Fujitsu Towns Sound");
			if (SubType.Data1 == WAVE_FORMAT_MICRONAS) return TEXT("Micronas");
			if (SubType.Data1 == WAVE_FORMAT_MICRONAS_CELP833) return TEXT("Micronas CELP833");
			if (SubType.Data1 == WAVE_FORMAT_BTV_DIGITAL) return TEXT("Brooktree Digital");
			if (SubType.Data1 == WAVE_FORMAT_INTEL_MUSIC_CODER) return TEXT("Intel Music Coder");
			if (SubType.Data1 == WAVE_FORMAT_INDEO_AUDIO) return TEXT("Indeo Audio");
			if (SubType.Data1 == WAVE_FORMAT_QDESIGN_MUSIC) return TEXT("QDesign Music");
			if (SubType.Data1 == WAVE_FORMAT_ON2_VP7_AUDIO) return TEXT("On2 VP7");
			if (SubType.Data1 == WAVE_FORMAT_ON2_VP6_AUDIO) return TEXT("On2 VP6");
			if (SubType.Data1 == WAVE_FORMAT_VME_VMPCM) return TEXT("AT&T VME VMPCM");
			if (SubType.Data1 == WAVE_FORMAT_TPC) return TEXT("AT&T TPC");
			if (SubType.Data1 == WAVE_FORMAT_LIGHTWAVE_LOSSLESS) return TEXT("Lightwave Lossless");
			if (SubType.Data1 == WAVE_FORMAT_OLIGSM) return TEXT("Olivetti GSM");
			if (SubType.Data1 == WAVE_FORMAT_OLIADPCM) return TEXT("Olivetti ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_OLICELP) return TEXT("Olivetti CELP");
			if (SubType.Data1 == WAVE_FORMAT_OLISBC) return TEXT("Olivetti SBC");
			if (SubType.Data1 == WAVE_FORMAT_OLIOPR) return TEXT("Olivetti OPR");
			if (SubType.Data1 == WAVE_FORMAT_LH_CODEC) return TEXT("Lernout & Hauspie");
			if (SubType.Data1 == WAVE_FORMAT_LH_CODEC_CELP) return TEXT("Lernout & Hauspie CELP");
			if (SubType.Data1 == WAVE_FORMAT_LH_CODEC_SBC8) return TEXT("Lernout & Hauspie SBC8");
			if (SubType.Data1 == WAVE_FORMAT_LH_CODEC_SBC12) return TEXT("Lernout & Hauspie SBC12");
			if (SubType.Data1 == WAVE_FORMAT_LH_CODEC_SBC16) return TEXT("Lernout & Hauspie SBC16");
			if (SubType.Data1 == WAVE_FORMAT_NORRIS) return TEXT("Norris");
			if (SubType.Data1 == WAVE_FORMAT_ISIAUDIO_2) return TEXT("ISIAudio 2");
			if (SubType.Data1 == WAVE_FORMAT_SOUNDSPACE_MUSICOMPRESS) return TEXT("AT&T SoundSpace Musicompress");
			if (SubType.Data1 == WAVE_FORMAT_MPEG_ADTS_AAC) return TEXT("MPEG ADT5 AAC");
			if (SubType.Data1 == WAVE_FORMAT_MPEG_RAW_AAC) return TEXT("MPEG RAW AAC");
			if (SubType.Data1 == WAVE_FORMAT_MPEG_LOAS) return TEXT("MPEG LOAS");
			if (SubType.Data1 == WAVE_FORMAT_NOKIA_MPEG_ADTS_AAC) return TEXT("Nokia MPEG ADT5 AAC");
			if (SubType.Data1 == WAVE_FORMAT_NOKIA_MPEG_RAW_AAC) return TEXT("Nokia MPEG RAW AAC");
			if (SubType.Data1 == WAVE_FORMAT_VODAFONE_MPEG_ADTS_AAC) return TEXT("Vodafone MPEG ADTS AAC");
			if (SubType.Data1 == WAVE_FORMAT_VODAFONE_MPEG_RAW_AAC) return TEXT("Vodafone MPEG RAW AAC");
			if (SubType.Data1 == WAVE_FORMAT_MPEG_HEAAC) return TEXT("MPEG HEAAC");
			if (SubType.Data1 == WAVE_FORMAT_VOXWARE_RT24_SPEECH) return TEXT("voxware RT24 Speech");
			if (SubType.Data1 == WAVE_FORMAT_SONICFOUNDRY_LOSSLESS) return TEXT("Sonic Foundry Lossless");
			if (SubType.Data1 == WAVE_FORMAT_INNINGS_TELECOM_ADPCM) return TEXT("Innings ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_LUCENT_SX8300P) return TEXT("Lucent SX8300P");
			if (SubType.Data1 == WAVE_FORMAT_LUCENT_SX5363S) return TEXT("Lucent SX5363S");
			if (SubType.Data1 == WAVE_FORMAT_CUSEEME) return TEXT("CUSeeMe");
			if (SubType.Data1 == WAVE_FORMAT_NTCSOFT_ALF2CM_ACM) return TEXT("NTCSoft ALF2CM ACM");
			if (SubType.Data1 == WAVE_FORMAT_DVM) return TEXT("FAST Multimedia DVM");
			if (SubType.Data1 == WAVE_FORMAT_DTS2) return TEXT("DTS2");
			if (SubType.Data1 == WAVE_FORMAT_MAKEAVIS) return TEXT("MAKEAVIS");
			if (SubType.Data1 == WAVE_FORMAT_DIVIO_MPEG4_AAC) return TEXT("Divio MPEG-4 AAC");
			if (SubType.Data1 == WAVE_FORMAT_NOKIA_ADAPTIVE_MULTIRATE) return TEXT("Nokia Adaptive Multirate");
			if (SubType.Data1 == WAVE_FORMAT_DIVIO_G726) return TEXT("Divio G726");
			if (SubType.Data1 == WAVE_FORMAT_LEAD_SPEECH) return TEXT("LEAD Speech");
			if (SubType.Data1 == WAVE_FORMAT_LEAD_VORBIS) return TEXT("LEAD Vorbis");
			if (SubType.Data1 == WAVE_FORMAT_WAVPACK_AUDIO) return TEXT("xiph.org WavPack");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_1) return TEXT("Ogg Vorbis Mode 1");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_2) return TEXT("Ogg Vorbis Mode 2");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_3) return TEXT("Ogg Vorbis Mode 3");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_1_PLUS) return TEXT("Ogg Vorbis Mode 1 Plus");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_2_PLUS) return TEXT("Ogg Vorbis Mode 2 Plus");
			if (SubType.Data1 == WAVE_FORMAT_OGG_VORBIS_MODE_3_PLUS) return TEXT("Ogg Vorbis Mode 3 Plus");
			if (SubType.Data1 == WAVE_FORMAT_3COM_NBX) return TEXT("3COM NBX");
			if (SubType.Data1 == WAVE_FORMAT_FAAD_AAC) return TEXT("FAAD AAC");
			if (SubType.Data1 == WAVE_FORMAT_AMR_NB) return TEXT("AMR Narrowband");
			if (SubType.Data1 == WAVE_FORMAT_AMR_WB) return TEXT("AMR Wideband");
			if (SubType.Data1 == WAVE_FORMAT_AMR_WP) return TEXT("AMR Wideband Plus");
			if (SubType.Data1 == WAVE_FORMAT_GSM_AMR_CBR) return TEXT("GSMA/3GPP CBR");
			if (SubType.Data1 == WAVE_FORMAT_GSM_AMR_VBR_SID) return TEXT("GSMA/3GPP VBR SID");
			if (SubType.Data1 == WAVE_FORMAT_COMVERSE_INFOSYS_G723_1) return TEXT("Converse Infosys G723.1");
			if (SubType.Data1 == WAVE_FORMAT_COMVERSE_INFOSYS_AVQSBC) return TEXT("Converse Infosys AVQSBC");
			if (SubType.Data1 == WAVE_FORMAT_COMVERSE_INFOSYS_SBC) return TEXT("Converse Infosys SBC");
			if (SubType.Data1 == WAVE_FORMAT_SYMBOL_G729_A) return TEXT("Symbol Technologies G729.A");
			if (SubType.Data1 == WAVE_FORMAT_VOICEAGE_AMR_WB) return TEXT("VoiceAge AMR Wideband");
			if (SubType.Data1 == WAVE_FORMAT_INGENIENT_G726) return TEXT("Ingenient G726");
			if (SubType.Data1 == WAVE_FORMAT_MPEG4_AAC) return TEXT("MPEG-4 AAC");
			if (SubType.Data1 == WAVE_FORMAT_ENCORE_G726) return TEXT("Encore G726");
			if (SubType.Data1 == WAVE_FORMAT_ZOLL_ASAO) return TEXT("ZOLL Medical ASAO");
			if (SubType.Data1 == WAVE_FORMAT_SPEEX_VOICE) return TEXT("xiph.org Speex Voice");
			if (SubType.Data1 == WAVE_FORMAT_VIANIX_MASC) return TEXT("Vianix MASC");
			if (SubType.Data1 == WAVE_FORMAT_WM9_SPECTRUM_ANALYZER) return TEXT("Windows Media 9 Spectrum Analyzer");
			if (SubType.Data1 == WAVE_FORMAT_WMF_SPECTRUM_ANAYZER) return TEXT("Windows Media Foundation Spectrum Analyzer");
			if (SubType.Data1 == WAVE_FORMAT_GSM_610) return TEXT("GSM 610");
			if (SubType.Data1 == WAVE_FORMAT_GSM_620) return TEXT("GSM 620");
			if (SubType.Data1 == WAVE_FORMAT_GSM_660) return TEXT("GSM 660");
			if (SubType.Data1 == WAVE_FORMAT_GSM_690) return TEXT("GSM 690");
			if (SubType.Data1 == WAVE_FORMAT_GSM_ADAPTIVE_MULTIRATE_WB) return TEXT("GSM Adaptive Multirate Wideband");
			if (SubType.Data1 == WAVE_FORMAT_POLYCOM_G722) return TEXT("Polycom G722");
			if (SubType.Data1 == WAVE_FORMAT_POLYCOM_G728) return TEXT("Polycom G728");
			if (SubType.Data1 == WAVE_FORMAT_POLYCOM_G729_A) return TEXT("Polycom G729.A");
			if (SubType.Data1 == WAVE_FORMAT_POLYCOM_SIREN) return TEXT("Polycom Siren");
			if (SubType.Data1 == WAVE_FORMAT_GLOBAL_IP_ILBC) return TEXT("Global IP ILBC");
			if (SubType.Data1 == WAVE_FORMAT_RADIOTIME_TIME_SHIFT_RADIO) return TEXT("RadioTime");
			if (SubType.Data1 == WAVE_FORMAT_NICE_ACA) return TEXT("Nice Systems ACA");
			if (SubType.Data1 == WAVE_FORMAT_NICE_ADPCM) return TEXT("Nice Systems ADPCM");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G721) return TEXT("Vocord G721");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G726) return TEXT("Vocord G726");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G722_1) return TEXT("Vocord G722.1");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G728) return TEXT("Vocord G728");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G729) return TEXT("Vocord G729");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G729_A) return TEXT("Vocord G729.A");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_G723_1) return TEXT("Vocord G723.1");
			if (SubType.Data1 == WAVE_FORMAT_VOCORD_LBC) return TEXT("Vocord LBC");
			if (SubType.Data1 == WAVE_FORMAT_NICE_G728) return TEXT("Nice Systems G728");
			if (SubType.Data1 == WAVE_FORMAT_FRACE_TELECOM_G729) return TEXT("France Telecom G729");
			if (SubType.Data1 == WAVE_FORMAT_CODIAN) return TEXT("CODIAN");
			if (SubType.Data1 == WAVE_FORMAT_FLAC) return TEXT("flac.sourceforge.net");
		}

		// unknown type
		return FString::Printf(TEXT("%s (%s)"), *GuidToString(SubType), *FourccToString(SubType.Data1));
	}


	FString TopologyStatusToString(MF_TOPOSTATUS Status)
	{
		switch (Status)
		{
		case MF_TOPOSTATUS_ENDED: return TEXT("Ended");
		case MF_TOPOSTATUS_INVALID: return TEXT("Invalid");
		case MF_TOPOSTATUS_READY: return TEXT("Ready");
		case MF_TOPOSTATUS_SINK_SWITCHED: return TEXT("Sink Switched");
		case MF_TOPOSTATUS_STARTED_SOURCE: return TEXT("Started Source");
		case MF_TOPOSTATUS_DYNAMIC_CHANGED: return TEXT("Dynamic Changed");

		default:
			return FString::Printf(TEXT("Unknown status %i"), (int32)Status);
		}
	}
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
