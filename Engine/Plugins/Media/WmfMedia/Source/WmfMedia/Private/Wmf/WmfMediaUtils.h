// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"


namespace WmfMedia
{
	/**
	 * Convert a media attribute identifier to string.
	 *
	 * @param Guid The unique identifier of the attribute.
	 * @return The corresponding name string.
	 */
	FString AttributeToString(const GUID& Guid);

	/**
	 * Convert a capture device role to string.
	 *
	 * @param Role The device role to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, GuidToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString CaptureDeviceRoleToString(ERole Role);

	/**
	 * Copy an attribute from one attribute collection to another.
	 *
	 * @param Src The attribute collection to copy from.
	 * @param Dest The attribute collection to copy to.
	 * @param Key The key of the attribute to copy.
	 * @return Result code.
	 */
	HRESULT CopyAttribute(IMFAttributes* Src, IMFAttributes* Dest, const GUID& Key);

	/**
	 * Create an output media type for the given input media type.
	 *
	 * @param InputType The input type to create the output type for.
	 * @param AllowNonStandardCodecs Whether non-standard codecs should be allowed.
	 * @param Whether the media source is a video capture device (there are some limitations with those).
	 * @return The output type, or NULL on error.
	 */
	TComPtr<IMFMediaType> CreateOutputType(IMFMediaType& InputType, bool AllowNonStandardCodecs, bool IsVideoDevice);

	/**
	 * Dump information about the given media type to the output log.
	 *
	 * @param MediaType The media type.
	 */
	FString DumpAttributes(IMFAttributes& Attributes);

	/**
	 * Enumerate the available capture devices.
	 *
	 * The supported device types are MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID for audio
	 * capture devices and MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID for video capture devices.
	 *
	 * @param DeviceType The type of devices to enumerate.
	 * @param OutDevices Will contain the collection of found device source activators.
	 */
	void EnumerateCaptureDevices(GUID DeviceType, TArray<TComPtr<IMFActivate>>& OutDevices);

	/**
	 * Convert an MF_MT_AM_FORMAT_TYPE value to string.
	 *
	 * @param FormatType The unique identifier of the format type.
	 * @return The corresponding string.
	 */
	FString FormatTypeToString(GUID FormatType);

	/**
	 * Convert a floating point frame rate to an integer ratio.
	 *
	 * @param OutNumerator Will contain the ratio's numerator.
	 * @param OutDenominator Will contain the ratio's denominator.
	 * @return true on success, false if the frame rate cannot be represented as a ratio.
	 * @see RatioToFrameRate
	 */
	bool FrameRateToRatio(float FrameRate, int32& OutNumerator, int32& OutDenominator);

	/** 
	 * Convert a FOURCC code to string.
	 *
	 * @param Fourcc The code to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, GuidToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString FourccToString(unsigned long Fourcc);

	/**
	 * Get information about the given capture device.
	 *
	 * @param Device The device.
	 * @param OutDisplayName Will contain the device's human readable name.
	 * @param OutInfo Will contain a device specific information string.
	 * @param OutSoftwareDevice Will contain a flag indicating whether the device is software.
	 * @param OutUrl Will contain the device's media URL string.
	 * @return true on success, false otherwise.
	 */
	bool GetCaptureDeviceInfo(IMFActivate& Device, FText& OutDisplayName, FString& OutInfo, bool& OutSoftwareDevice, FString& OutUrl);

	/**
	 * Get the list of supported media types for the specified major type.
	 *
	 * @param MajorType The major type to query, i.e. audio or video.
	 * @return Media type collection.
	 * @see IsSupportedMajorType
	 */
	const TArray<TComPtr<IMFMediaType>>& GetSupportedMediaTypes(const GUID& MajorType);

	/**
	 * Get the playback topology object from the given media event.
	 *
	 * @param Event The event object.
	 * @param OutTopology Will contain the topology.
	 * @return Result code.
	 */
	HRESULT GetTopologyFromEvent(IMFMediaEvent& Event, TComPtr<IMFTopology>& OutTopology);

	/**
	 * Convert a Windows GUID to string.
	 *
	 * @param Guid The GUID to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString GuidToString(const GUID& Guid);

	/**
	 * Convert a video interlace mode to string.
	 *
	 * @param Mode The interlace mode to convert.
	 * @return The corresponding string.
	 */
	FString InterlaceModeToString(MFVideoInterlaceMode Mode);

	/**
	 * Check whether the given major media type is supported.
	 *
	 * @param Type The major type to check.
	 * @return true if the type is supported, false otherwise.
	 * @see GetSupportedMediaTypes
	 */
	bool IsSupportedMajorType(const GUID& MajorType);

	/**
	 * Convert a media major type to string.
	 *
	 * @param MajorType The major type GUID to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MarkerTypeToString, MediaEventToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString MajorTypeToString(const GUID& MajorType);

	/**
	 * Convert a stream sink marker type to string.
	 *
	 * @param MarkerType The marker type to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MajorTypeToString, MediaEventToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString MarkerTypeToString(MFSTREAMSINK_MARKER_TYPE MarkerType);

	/**
	 * Convert a media event to string.
	 *
	 * @param Event The event code to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MajorTypeToString, MarkerTypeToString, ResultToString, SubTypeToString, TopologyStatusToString
	 */
	FString MediaEventToString(MediaEventType Event);

	/**
	 * Convert an integer ratio to a floating point frame rate.
	 *
	 * @param Numerator The ratio's numerator.
	 * @param Denominator The ratio's denominator.
	 * @return The corresponding floating point value.
	 * @see FrameRateToRatio
	 */
	float RatioToFrameRate(int32 Numerator, int32 Denominator);

	/**
	 * Resolve a media source from an archive or URL.
	 *
	 * @param Archive The archive to read media data from (optional).
	 * @param Url The media source URL.
	 * @param Precache Whether to precache media into RAM if URL is a local file.
	 * @return The media source object, or NULL if it couldn't be resolved.
	 */
	TComPtr<IMFMediaSource> ResolveMediaSource(TSharedPtr<FArchive, ESPMode::ThreadSafe> Archive, const FString& Url, bool Precache);

	/**
	 * Convert an WMF HRESULT code to string.
	 *
	 * @param Result The result code to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, SubTypeToString, TopologyStatusToString
	 */
	FString ResultToString(HRESULT Result);

	/**
	 * Convert a media sub-type to string.
	 *
	 * @param SubType The sub-type GUID to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, ResultToString, TopologyStatusToString
	 */
	FString SubTypeToString(const GUID& SubType);

	/**
	 * Convert a WMF topology status to string.
	 *
	 * @param Status The status to convert.
	 * @return The corresponding string.
	 * @see CaptureDeviceRoleToString, FourccToString, GuidToString, MajorTypeToString, MarkerTypeToString, MediaEventToString, ResultToString, SubTypeToString
	 */
	FString TopologyStatusToString(MF_TOPOSTATUS Status);
}


#include "Windows/HideWindowsPlatformTypes.h"

#endif //WMFMEDIA_SUPPORTED_PLATFORM
