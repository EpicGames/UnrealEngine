// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaCodec/WmfMediaCodecManager.h"

 #include "WmfMediaSettings.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

bool WmfMediaCodecManager::IsCodecSupported(const GUID& InMajorType, const GUID& InSubType) const
{
	for (const TUniquePtr<IWmfMediaCodec>& Codec : RegisteredCodec)
	{
		if (Codec->IsCodecSupported(InMajorType, InSubType))
		{
			return true;
		}
	}
	return false;
}

bool WmfMediaCodecManager::SetVideoFormat(const GUID& InSubType, GUID& OutVideoFormat) const
{
	for (const TUniquePtr<IWmfMediaCodec>& Codec : RegisteredCodec)
	{
		if (Codec->SetVideoFormat(InSubType, OutVideoFormat))
		{
			return true;
		}
	}
	return false;
}

bool WmfMediaCodecManager::SetupDecoder(
	const GUID& InMajorType,
	const GUID& InSubType,
	TComPtr<IMFTopologyNode>& InDecoderNode,
	TComPtr<IMFTransform>& InTransform)
{
	for (const TUniquePtr<IWmfMediaCodec>& Codec : RegisteredCodec)
	{
		if (Codec->SetupDecoder(InMajorType, InSubType, InDecoderNode, InTransform))
		{
			return true;
		}
	}
	return false;
}

void WmfMediaCodecManager::AddCodec(TUniquePtr<IWmfMediaCodec> InCodec)
{
	check(InCodec.IsValid());
	if (InCodec->IsHardwareAccelerated())
	{
		GetMutableDefault<UWmfMediaSettings>()->EnableHardwareAcceleratedCodecRegistered();
	}
	RegisteredCodec.Add(MoveTemp(InCodec));
}

#endif // WMFMEDIA_SUPPORTED_PLATFORM