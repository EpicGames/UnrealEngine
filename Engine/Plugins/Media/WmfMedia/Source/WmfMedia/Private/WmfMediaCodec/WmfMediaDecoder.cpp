// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaCodec/WmfMediaDecoder.h"

#include "IWmfMediaModule.h"
#include "Windows/AllowWindowsPlatformTypes.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include <d3d11.h>

WmfMediaDecoder::WmfMediaDecoder()
	: RefCount(0),
	InputType(NULL),
	OutputType(NULL),
	ImageWidthInPixels(0),
	ImageHeightInPixels(0),
	FrameRate{ 0 },
	InputImageSize(0),
	OutputImageSize(0),
	InternalTimeStamp(0),
	SampleDuration(0),
	bIsExternalBufferEnabled(false)
{
}

WmfMediaDecoder::~WmfMediaDecoder()
{
	RemoveDecoderFromMap();

	// Clean up external buffer pool.
	for (TArray<uint8>*& pBuffer : ExternalBufferPool)
	{
		if (pBuffer != nullptr)
		{
			delete pBuffer;
			pBuffer = nullptr;
		}
	}

}

ULONG WmfMediaDecoder::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}


ULONG WmfMediaDecoder::Release()
{
	ULONG Count = FPlatformAtomics::InterlockedDecrement(&RefCount);
	if (Count == 0)
	{
		delete this;
	}
	return Count;
}


HRESULT WmfMediaDecoder::GetStreamLimits(DWORD* pdwInputMinimum, DWORD* pdwInputMaximum, DWORD* pdwOutputMinimum, DWORD* pdwOutputMaximum)
{
	if ((pdwInputMinimum == NULL) ||
		(pdwInputMaximum == NULL) ||
		(pdwOutputMinimum == NULL) ||
		(pdwOutputMaximum == NULL))
	{
		return E_POINTER;
	}

	*pdwInputMinimum = 1;
	*pdwInputMaximum = 1;
	*pdwOutputMinimum = 1;
	*pdwOutputMaximum = 1;

	return S_OK;
}

HRESULT WmfMediaDecoder::GetStreamCount(DWORD* pcInputStreams, DWORD* pcOutputStreams)
{
	if ((pcInputStreams == NULL) || (pcOutputStreams == NULL))

	{
		return E_POINTER;
	}

	*pcInputStreams = 1;
	*pcOutputStreams = 1;

	return S_OK;
}


HRESULT WmfMediaDecoder::GetStreamIDs(DWORD /*dwInputIDArraySize*/, DWORD* /*pdwInputIDs*/, DWORD /*dwOutputIDArraySize*/, DWORD* /*pdwOutputIDs*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::GetInputStreamInfo(DWORD dwInputStreamID, MFT_INPUT_STREAM_INFO* pStreamInfo)
{
	if (pStreamInfo == NULL)
	{
		return E_POINTER;
	}

	if (dwInputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	pStreamInfo->hnsMaxLatency = 0;
	pStreamInfo->dwFlags = 0;
	pStreamInfo->cbSize = 1;
	pStreamInfo->cbMaxLookahead = 0;
	pStreamInfo->cbAlignment = 1;

	return S_OK;
}



HRESULT WmfMediaDecoder::GetOutputStreamInfo(DWORD dwOutputStreamID, MFT_OUTPUT_STREAM_INFO* pStreamInfo)
{
	if (pStreamInfo == NULL)
	{
		return E_POINTER;
	}

	if (dwOutputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	FScopeLock Lock(&CriticalSection);

	pStreamInfo->dwFlags =
		MFT_OUTPUT_STREAM_PROVIDES_SAMPLES |
		MFT_OUTPUT_STREAM_WHOLE_SAMPLES |
		MFT_OUTPUT_STREAM_SINGLE_SAMPLE_PER_BUFFER |
		MFT_OUTPUT_STREAM_FIXED_SAMPLE_SIZE;

	pStreamInfo->cbSize = 0;
	pStreamInfo->cbAlignment = 0;

	return S_OK;
}


HRESULT WmfMediaDecoder::GetAttributes(IMFAttributes** ppAttributes)
{
	TComPtr<IMFAttributes> Attributes;
	HRESULT hr = MFCreateAttributes(&Attributes, 1);

	if (SUCCEEDED(hr))
	{
		// MF_SA_D3D11_AWARE Normally defined for Windows 8+ platform, since we also support win7, it will not be defined.
		const GUID UE_MF_SA_D3D11_AWARE = { 0x206b4fc8, 0xfcf9, 0x4c51, { 0xaf, 0xe3, 0x97, 0x64, 0x36, 0x9e, 0x33, 0xa0 } };
		Attributes->SetUINT32(UE_MF_SA_D3D11_AWARE, TRUE);
		*ppAttributes = Attributes;
		(*ppAttributes)->AddRef();
		return S_OK;
	}
	else
	{
		return E_NOTIMPL;
	}
}



HRESULT WmfMediaDecoder::GetInputStreamAttributes(DWORD /*dwInputStreamID*/, IMFAttributes** /*ppAttributes*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::GetOutputStreamAttributes(DWORD /*dwOutputStreamID*/, IMFAttributes** /*ppAttributes*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::DeleteInputStream(DWORD /*dwStreamID*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::AddInputStreams(DWORD /*cStreams*/, DWORD* /*adwStreamIDs*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::GetInputAvailableType(DWORD /*dwInputStreamID*/, DWORD /*dwTypeIndex*/, IMFMediaType** /*ppType*/)
{
	return MF_E_NO_MORE_TYPES;
}


HRESULT WmfMediaDecoder::SetInputType(DWORD dwInputStreamID, IMFMediaType* pType, DWORD dwFlags)
{
	if (dwInputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	if (dwFlags & ~MFT_SET_TYPE_TEST_ONLY)
	{
		return E_INVALIDARG;
	}

	HRESULT hr = S_OK;

	FScopeLock Lock(&CriticalSection);

	BOOL bReallySet = ((dwFlags & MFT_SET_TYPE_TEST_ONLY) == 0);

	if (HasPendingOutput())
	{
		hr = MF_E_TRANSFORM_CANNOT_CHANGE_MEDIATYPE_WHILE_PROCESSING;
	}

	if (SUCCEEDED(hr))
	{
		if (pType)
		{
			hr = OnCheckInputType(pType);
		}
	}

	if (SUCCEEDED(hr))
	{
		if (bReallySet)
		{
			hr = OnSetInputType(pType);
		}
	}

	return hr;
}


HRESULT WmfMediaDecoder::SetOutputType(DWORD dwOutputStreamID, IMFMediaType* pType, DWORD dwFlags)
{
	if (dwOutputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	if (dwFlags & ~MFT_SET_TYPE_TEST_ONLY)
	{
		return E_INVALIDARG;
	}

	HRESULT hr = S_OK;

	FScopeLock Lock(&CriticalSection);

	BOOL bReallySet = ((dwFlags & MFT_SET_TYPE_TEST_ONLY) == 0);

	if (HasPendingOutput())
	{
		hr = MF_E_TRANSFORM_CANNOT_CHANGE_MEDIATYPE_WHILE_PROCESSING;
	}

	if (SUCCEEDED(hr))
	{
		if (pType)
		{
			hr = OnCheckOutputType(pType);
		}
	}

	if (SUCCEEDED(hr))
	{
		if (bReallySet)
		{
			hr = OnSetOutputType(pType);
		}
	}

	return hr;
}


HRESULT WmfMediaDecoder::GetInputCurrentType(DWORD dwInputStreamID, IMFMediaType** ppType)
{
	if (ppType == NULL)
	{
		return E_POINTER;
	}

	if (dwInputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	if (!InputType)
	{
		hr = MF_E_TRANSFORM_TYPE_NOT_SET;
	}

	if (SUCCEEDED(hr))
	{
		*ppType = InputType;
		(*ppType)->AddRef();
	}

	return hr;
}


HRESULT WmfMediaDecoder::GetOutputCurrentType(DWORD dwOutputStreamID, IMFMediaType** ppType)
{
	if (ppType == NULL)
	{
		return E_POINTER;
	}

	if (dwOutputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	if (!OutputType)
	{
		hr = MF_E_TRANSFORM_TYPE_NOT_SET;
	}

	if (SUCCEEDED(hr))
	{
		*ppType = OutputType;
		(*ppType)->AddRef();
	}

	return hr;
}


HRESULT WmfMediaDecoder::GetInputStatus(DWORD dwInputStreamID, DWORD* pdwFlags)
{
	if (pdwFlags == NULL)
	{
		return E_POINTER;
	}

	if (dwInputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	FScopeLock Lock(&CriticalSection);

	if (HasPendingOutput())
	{
		*pdwFlags = MFT_INPUT_STATUS_ACCEPT_DATA;
	}
	else
	{
		*pdwFlags = 0;
	}

	return S_OK;
}


HRESULT WmfMediaDecoder::GetOutputStatus(DWORD* pdwFlags)
{
	if (pdwFlags == NULL)
	{
		return E_POINTER;
	}

	FScopeLock Lock(&CriticalSection);

	if (HasPendingOutput())
	{
		*pdwFlags = MFT_OUTPUT_STATUS_SAMPLE_READY;
	}
	else
	{
		*pdwFlags = 0;
	}

	return S_OK;
}


HRESULT WmfMediaDecoder::SetOutputBounds(LONGLONG /*hnsLowerBound*/, LONGLONG /*hnsUpperBound*/)
{
	return E_NOTIMPL;
}


HRESULT WmfMediaDecoder::ProcessEvent(DWORD /*dwInputStreamID*/, IMFMediaEvent* pEvent)
{
	MediaEventType EventType = MEUnknown;
	pEvent->GetType(&EventType);

	if (EventType == MEStreamThinMode)
	{
		PROPVARIANT Value;
		pEvent->GetValue(&Value);
		return S_OK;
	}
	else
	{
		return E_NOTIMPL;
	}
}


HRESULT WmfMediaDecoder::OnCheckOutputType(IMFMediaType* InMediaType)
{
	if (OutputType)
	{
		DWORD dwFlags = 0;
		if (S_OK == OutputType->IsEqual(InMediaType, &dwFlags))
		{
			return S_OK;
		}
		else
		{
			return MF_E_INVALIDTYPE;
		}
	}

	if (InputType == NULL)
	{
		return MF_E_TRANSFORM_TYPE_NOT_SET;
	}

	HRESULT hr = S_OK;
	BOOL bMatch = FALSE;

	TComPtr<IMFMediaType> OurType;

	hr = GetOutputAvailableType(0, 0, &OurType);

	if (SUCCEEDED(hr))
	{
		hr = OurType->Compare(InMediaType, MF_ATTRIBUTES_MATCH_OUR_ITEMS, &bMatch);
	}

	if (SUCCEEDED(hr))
	{
		if (!bMatch)
		{
			hr = MF_E_INVALIDTYPE;
		}
	}

	return hr;
}


HRESULT WmfMediaDecoder::OnSetOutputType(IMFMediaType* InMediaType)
{
	OutputType = InMediaType;
	OutputType->AddRef();

	return S_OK;
}


HRESULT WmfMediaDecoder::OnDiscontinuity()
{
	InternalTimeStamp = 0;
	EmplyQueues();
	return S_OK;
}


HRESULT WmfMediaDecoder::OnFlush()
{
	OnDiscontinuity();
	return S_OK;
}

TArray<uint8>* WmfMediaDecoder::AllocateExternalBuffer(uint64 InTimeStamp, int32 InSize)
{
	FScopeLock Lock(&BufferCriticalSection);

	// Do we already have this buffer?
	TArray<uint8>** ppArray = MapTimeStampToExternalBuffer.Find(InTimeStamp);
	TArray<uint8>* pBuffer = (ppArray != nullptr) ? (*ppArray) : nullptr;
	if (pBuffer == nullptr)
	{
		// No. Do we have any buffers we can reuse?
		if (ExternalBufferPool.Num() > 0)
		{
			// Look for a buffer with the right size.
			int32 Index = 0;
			for (Index = 0; Index < ExternalBufferPool.Num(); ++Index)
			{
				pBuffer = ExternalBufferPool[Index];
				if (pBuffer->Num() == InSize)
				{
					break;
				}
			}

			// If there is none, then just use the first one.
			if (Index >= ExternalBufferPool.Num())
			{
				Index = 0;
				pBuffer = ExternalBufferPool[Index];
			}

			// Remove from the pool.
			ExternalBufferPool.RemoveAtSwap(Index);
		}
		else
		{
			// No free buffers. Just create a new one.
			UE_LOGF(LogWmfMedia, VeryVerbose, "WmfMediaDecoder::AllocateExternalBuffer new array.");
			pBuffer = new TArray<uint8>();
		}

		// Associate the time stamp wth this buffer.
		MapTimeStampToExternalBuffer.Emplace(InTimeStamp, pBuffer);
	}

	// Is this buffer the right size?
	if (pBuffer->Num() != InSize)
	{
		// Nope.
		// Remove old buffer from map.
		uint8* pData = pBuffer->GetData();
		FScopeLock MapLock(&GetMapBufferCriticalSection());
		TMap<uint8*, WmfMediaDecoder*>& Map = GetMapBufferToDecoder();
		Map.Remove(pData);

		// Allocate new size.
		UE_LOGF(LogWmfMedia, VeryVerbose, "WmfMediaDecoder::AllocateExternalBuffer new buffer size:%d old:%d", InSize, pBuffer->Num());
		pBuffer->SetNum(InSize);

		// Update map.
		pData = pBuffer->GetData();
		Map.Add(pData, this);
	}

	return pBuffer;
}

bool WmfMediaDecoder::IsExternalBufferSupported() const
{
	return false;
}

void WmfMediaDecoder::EnableExternalBuffer(bool bInEnable)
{
	bIsExternalBufferEnabled = bInEnable;
}

bool WmfMediaDecoder::GetExternalBuffer(TArray<uint8>& InBuffer, uint64 TimeStamp)
{
	FScopeLock Lock(&BufferCriticalSection);

	TArray<uint8>* pBuffer = nullptr;
	MapTimeStampToExternalBuffer.RemoveAndCopyValue(TimeStamp, pBuffer);

	if (pBuffer != nullptr)
	{
		// Move the buffer out.
		InBuffer = MoveTemp(*pBuffer);

		// Put this array back in the pool.
		ExternalBufferPool.Add(pBuffer);
		return true;
	}

	return false;
}

void WmfMediaDecoder::ReturnExternalBuffer(TArray<uint8>& InBuffer)
{
	// Get the decoder for this buffer.
	FScopeLock Lock(&GetMapBufferCriticalSection());
	TMap<uint8*, WmfMediaDecoder*>& Map = GetMapBufferToDecoder();
	WmfMediaDecoder** ppDecoder = Map.Find(InBuffer.GetData());

	// Return the buffer to the decoder.
	if ((ppDecoder != nullptr) && ((*ppDecoder)!= nullptr))
	{
		(*ppDecoder)->ReturnExternalBufferInternal(InBuffer);
	}
}

void WmfMediaDecoder::ReturnExternalBufferInternal(TArray<uint8>& InBuffer)
{
	FScopeLock Lock(&BufferCriticalSection);

	/** Find an empty TArray */
	TArray<uint8>* pEmptyBuffer = nullptr;
	for (TArray<uint8>* pBuffer : ExternalBufferPool)
	{
		if (pBuffer->Num() == 0)
		{
			pEmptyBuffer = pBuffer;
			break;
		}
	}

	/** If there is none, then just create a new one. */
	if (pEmptyBuffer == nullptr)
	{
		pEmptyBuffer = new TArray<uint8>();
		ExternalBufferPool.Add(pEmptyBuffer);
		UE_LOGF(LogWmfMedia, VeryVerbose, "WmfMediaDecoder::ReturnExternalBufferInternal new array.");
	}

	// Move the data out to our buffer.
	*pEmptyBuffer = MoveTemp(InBuffer);
}

void WmfMediaDecoder::RemoveDecoderFromMap()
{
	if (bIsExternalBufferEnabled)
	{
		FScopeLock Lock(&GetMapBufferCriticalSection());
		TMap<uint8*, WmfMediaDecoder*>& Map = GetMapBufferToDecoder();
		for (TMap<uint8*, WmfMediaDecoder*>::TIterator It = Map.CreateIterator(); It; ++It)
		{
			if (It.Value() == this)
			{
				It.RemoveCurrent();
			}
		}
	}
}

FCriticalSection& WmfMediaDecoder::GetMapBufferCriticalSection()
{
	static FCriticalSection MapbufferCriticalSection;
	return MapbufferCriticalSection;
}

TMap<uint8*, WmfMediaDecoder*>& WmfMediaDecoder::GetMapBufferToDecoder()
{
	static TMap<uint8*, WmfMediaDecoder*> Map;
	return Map;
}

void WmfMediaDecoder::EmplyQueues()
{
	InputQueue.Empty();
	OutputQueue.Empty();
}


HRESULT WmfMediaDecoder::ProcessInput(DWORD dwInputStreamID, IMFSample* pSample, DWORD dwFlags)
{
	if (pSample == NULL)
	{
		return E_POINTER;
	}

	if (dwInputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	if (dwFlags != 0)
	{
		return E_INVALIDARG;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	if (!InputType || !OutputType)
	{
		hr = MF_E_NOTACCEPTING;
	}
	else if (!OutputQueue.IsEmpty())
	{
		hr = MF_E_NOTACCEPTING;
	}

	TComPtr<IMFMediaBuffer> MediaBuffer;

	if (SUCCEEDED(hr))
	{
		hr = pSample->ConvertToContiguousBuffer(&MediaBuffer);
	}

	BYTE *Data = nullptr;
	DWORD DataSize = 0;

	if (SUCCEEDED(hr))
	{
		hr = MediaBuffer->Lock(&Data, NULL, &DataSize);
	}

	if (SUCCEEDED(hr))
	{
		LONGLONG SampleTimeStamp = 0;
		if (FAILED(pSample->GetSampleTime(&SampleTimeStamp)))
		{
			SampleTimeStamp = InternalTimeStamp;
		}

		hr = InternalProcessInput(SampleTimeStamp, Data, DataSize);

		InternalTimeStamp += SampleDuration;
	}

	MediaBuffer->Unlock();

	return hr;
}

#endif // WMFMEDIA_SUPPORTED_PLATFORM

#include "Windows/HideWindowsPlatformTypes.h"

