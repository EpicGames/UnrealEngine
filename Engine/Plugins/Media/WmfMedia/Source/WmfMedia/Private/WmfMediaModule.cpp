// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaCommon.h"

#include "IMediaCaptureSupport.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

#include "IWmfMediaModule.h"

#include "WmfMediaCodec/WmfMediaCodecManager.h"

#if WMFMEDIA_SUPPORTED_PLATFORM
	#include "IMediaModule.h"
	#include "Interfaces/IPluginManager.h"
	#include "Modules/ModuleInterface.h"
	#include "Templates/SharedPointer.h"
	#include "ShaderCore.h"

	#include "Player/WmfMediaPlayer.h"
	#include "Wmf/WmfMediaUtils.h"

	#pragma comment(lib, "mf")
	#pragma comment(lib, "mfplat")
	#pragma comment(lib, "mfplay")
	#pragma comment(lib, "mfuuid")
	#pragma comment(lib, "shlwapi")
	#pragma comment(lib, "d3d11")

#endif


DEFINE_LOG_CATEGORY(LogWmfMedia);

// Defined here to allow forward declaration of WmfMediaCodecManager with smart pointer to work
IWmfMediaModule::~IWmfMediaModule()
{

}

/**
 * Implements the WmfMedia module.
 */
class FWmfMediaModule
	: public IMediaCaptureSupport
	, public IWmfMediaModule
{
public:

	/** Default constructor. */
	FWmfMediaModule()
		: Initialized(false)
	{ }

public:

	//~ IMediaCaptureDevices interface

	virtual void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM
		EnumerateCaptureDevices(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_AUDCAP_GUID, OutDeviceInfos);
#endif
	}

	virtual void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM
		EnumerateCaptureDevices(MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID, OutDeviceInfos);
#endif
	}

public:

	//~ IWmfMediaModule interface

	virtual bool IsInitialized() const override
	{
		return Initialized;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM
		if (!Initialized)
		{
			return nullptr;
		}

		return MakeShareable(new FWmfMediaPlayer(EventSink));
#else
		return nullptr;
#endif
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM

		// Maps virtual shader source directory /Plugin/WmfMedia to the plugin's actual Shaders directory.
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("WmfMedia"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/WmfMedia"), PluginShaderDir);

		// load required libraries
		if (!LoadRequiredLibraries())
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to load required Windows Media Foundation libraries");

			return;
		}

		// initialize Windows Media Foundation
		HRESULT Result = MFStartup(MF_VERSION);

		if (FAILED(Result))
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to initialize Windows Media Foundation, Error %i", Result);

			return;
		}

		// register capture device support
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterCaptureSupport(*this);
		}

		CodecManager = MakeUnique<WmfMediaCodecManager>();

		Initialized = true;

#endif //WMFMEDIA_SUPPORTED_PLATFORM
	}

	virtual void ShutdownModule() override
	{
#if WMFMEDIA_SUPPORTED_PLATFORM
		if (!Initialized)
		{
			return;
		}

		// unregister capture support
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterCaptureSupport(*this);
		}

		// shutdown Windows Media Foundation
		MFShutdown();

		Initialized = false;

#endif //WMFMEDIA_SUPPORTED_PLATFORM
	}

protected:

#if WMFMEDIA_SUPPORTED_PLATFORM

	/**
	 * Enumerate capture devices of the specified type.
	 *
	 * @param DeviceType The type of devices to enumerate.
	 * @param SoftwareOnly Whether to enumerate software devices only.
	 * @param OutDeviceInfo Will contain information about the devices.
	 */
	void EnumerateCaptureDevices(GUID DeviceType, TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos)
	{
		TArray<TComPtr<IMFActivate>> Devices;
		WmfMedia::EnumerateCaptureDevices(DeviceType, Devices);

		for (auto& Device : Devices)
		{
			FMediaCaptureDeviceInfo DeviceInfo;
			bool SoftwareDevice = false;

			if (!WmfMedia::GetCaptureDeviceInfo(*Device, DeviceInfo.DisplayName, DeviceInfo.Info, SoftwareDevice, DeviceInfo.Url))
			{
				continue;
			}

			if (DeviceType == MF_DEVSOURCE_ATTRIBUTE_SOURCE_TYPE_VIDCAP_GUID)
			{
				DeviceInfo.Type = SoftwareDevice ? EMediaCaptureDeviceType::VideoSoftware : EMediaCaptureDeviceType::Video;
			}
			else
			{
				DeviceInfo.Type = EMediaCaptureDeviceType::Audio;
			}

			OutDeviceInfos.Add(MoveTemp(DeviceInfo));
		}
	}

	/**
	 * Loads all required Windows libraries.
	 *
	 * @return true on success, false otherwise.
	 */
	bool LoadRequiredLibraries()
	{
		if (LoadLibraryW(TEXT("shlwapi.dll")) == nullptr)
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to load shlwapi.dll");

			return false;
		}

		if (LoadLibraryW(TEXT("mf.dll")) == nullptr)
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to load mf.dll");

			return false;
		}

		if (LoadLibraryW(TEXT("mfplat.dll")) == nullptr)
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to load mfplat.dll");

			return false;
		}

		if (LoadLibraryW(TEXT("mfplay.dll")) == nullptr)
		{
			UE_LOGF(LogWmfMedia, Log, "Failed to load mfplay.dll");

			return false;
		}

		return true;
	}

#endif //WMFMEDIA_SUPPORTED_PLATFORM

private:

	/** Whether the module has been initialized. */
	bool Initialized;
};


IMPLEMENT_MODULE(FWmfMediaModule, WmfMedia);
