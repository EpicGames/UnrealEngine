// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "guiddef.h"

#include "Microsoft/COMPointer.h"

struct IMFTopologyNode;
struct IMFTransform;

class IWmfMediaCodec
{
public:
	virtual ~IWmfMediaCodec() = default;
	virtual bool IsCodecSupported(const GUID& InMajorType, const GUID& InSubType) const { return false; }
	virtual bool SetVideoFormat(const GUID& InSubType, GUID& OutVideoFormat) const { return false;  }
	virtual bool SetupDecoder(
		const GUID& InMajorType,
		const GUID& InSubType,
		TComPtr<IMFTopologyNode>& InDecoderNode,
		TComPtr<IMFTransform>& InTransform)
	{
		return false;
	}
	virtual bool IsHardwareAccelerated() const { return false; }
};
