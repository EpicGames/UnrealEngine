// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IMediaEventSink;
class IMediaPlayer;

class WmfMediaCodecManager;

/**
 * Interface for the WmfMedia module.
 */
class IWmfMediaModule
	: public IModuleInterface
{
public:

	/**
	 * Is the Wmf media module initialized?
	 @return True if the module is initialized.
	 */
	virtual bool IsInitialized() const = 0;

	/** Get this module */
	static IWmfMediaModule* Get()
	{
		static const FName ModuleName = "WmfMedia";
		return FModuleManager::GetModulePtr<IWmfMediaModule>(ModuleName);
	}

	/**
	 * Creates a Windows Media Foundation based media player.
	 *
	 * @param EventSink The object that receives media events from the player.
	 * @return A new media player, or nullptr if a player couldn't be created.
	 */
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) = 0;
	WmfMediaCodecManager* GetCodecManager() { return CodecManager.Get(); }

public:

	/** Virtual destructor. */
	virtual ~IWmfMediaModule();

protected:
	/**  Codec manager which handle codec from other plugin. */
	TUniquePtr<WmfMediaCodecManager> CodecManager;
};
