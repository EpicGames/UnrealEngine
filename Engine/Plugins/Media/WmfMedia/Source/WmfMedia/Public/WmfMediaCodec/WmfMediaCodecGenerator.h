// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"
#include "IWmfMediaCodec.h"

template <class T>
class WmfMediaCodecGenerator : public IWmfMediaCodec
{
public:

	WmfMediaCodecGenerator(bool bInIsHardwareAccelerated)
	{
		bIsHardwareAccelerated = bInIsHardwareAccelerated;
	}

	virtual ~WmfMediaCodecGenerator() = default;

	virtual bool IsCodecSupported(const GUID& InMajorType, const GUID& InSubType) const override
	{
		if (InMajorType != T::GetMajorType())
		{
			return false;
		}

		return T::IsSupported(InSubType);
	}

	virtual bool SetVideoFormat(const GUID& InSubType, GUID& OutVideoFormat) const
	{
		return T::SetOutputFormat(InSubType, OutVideoFormat);
	}

	virtual bool SetupDecoder(
		const GUID& InMajorType,
		const GUID& InSubType,
		TComPtr<IMFTopologyNode>& InDecoderNode,
		TComPtr<IMFTransform>& InTransform)
	{
		if (!IsCodecSupported(InMajorType, InSubType))
		{
			return false;
		}

		HRESULT Result = ::MFCreateTopologyNode(MF_TOPOLOGY_TRANSFORM_NODE, &InDecoderNode);

		if (FAILED(Result))
		{
			return false;
		}

		InTransform = new T();
		return true;
	}

	virtual bool IsHardwareAccelerated() const
	{
		return bIsHardwareAccelerated;
	}
	
private:

	bool bIsHardwareAccelerated;
};
