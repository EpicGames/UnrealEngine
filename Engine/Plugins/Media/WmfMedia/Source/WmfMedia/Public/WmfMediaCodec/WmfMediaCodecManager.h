// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UniquePtr.h"

#include "WmfMediaCommon.h"

#include "IWmfMediaCodec.h"

#define UE_API WMFMEDIA_API

#if WMFMEDIA_SUPPORTED_PLATFORM

class WmfMediaCodecManager : public IWmfMediaCodec
{
public:
	WmfMediaCodecManager() = default;
	WmfMediaCodecManager(const WmfMediaCodecManager&) = delete;
	WmfMediaCodecManager(WmfMediaCodecManager&&) = delete;
	WmfMediaCodecManager& operator=(const WmfMediaCodecManager&) = delete;

	UE_API virtual bool IsCodecSupported(const GUID& InMajorType, const GUID& InSubType) const override;
	UE_API virtual bool SetVideoFormat(const GUID& InSubType, GUID& OutVideoFormat) const override;
	UE_API virtual bool SetupDecoder(
		const GUID& InMajorType,
		const GUID& InSubType,
		TComPtr<IMFTopologyNode>& InDecoderNode,
		TComPtr<IMFTransform>& InTransform) override;

	UE_API void AddCodec(TUniquePtr<IWmfMediaCodec> InCodec);

private:
	TArray<TUniquePtr<IWmfMediaCodec>> RegisteredCodec;
};

#else

class WmfMediaCodecManager
{

};

#endif // WMFMEDIA_SUPPORTED_PLATFORM

#undef UE_API
