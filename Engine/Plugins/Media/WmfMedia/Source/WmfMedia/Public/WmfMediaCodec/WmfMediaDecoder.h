// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCommon.h"

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"

#define UE_API WMFMEDIA_API

#if WMFMEDIA_SUPPORTED_PLATFORM

struct ID3D11DeviceContext;
struct ID3D11Device;

class WmfMediaDecoder : public IMFTransform
{
public:

	struct DataBuffer
	{
		TArray<unsigned char> Color;
		TArray<unsigned char> Alpha;
		LONGLONG TimeStamp;
	};

public:
	UE_API WmfMediaDecoder();
	UE_API virtual ~WmfMediaDecoder();

	static GUID GetMajorType() { return MFMediaType_Video; }

	UE_API virtual ULONG STDMETHODCALLTYPE AddRef() override;
	UE_API virtual ULONG STDMETHODCALLTYPE Release() override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetStreamLimits(DWORD* pdwInputMinimum, DWORD* pdwInputMaximum, DWORD* pdwOutputMinimum, DWORD* pdwOutputMaximum) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetStreamCount(DWORD* pcInputStreams, DWORD* pcOutputStreams) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetStreamIDs(DWORD dwInputIDArraySize, DWORD* pdwInputIDs, DWORD dwOutputIDArraySize, DWORD* pdwOutputIDs) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetInputStreamInfo(DWORD dwInputStreamID, MFT_INPUT_STREAM_INFO* pStreamInfo) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetOutputStreamInfo(DWORD dwOutputStreamID, MFT_OUTPUT_STREAM_INFO* pStreamInfo) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetAttributes(IMFAttributes** pAttributes) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetInputStreamAttributes(DWORD dwInputStreamID, IMFAttributes** ppAttributes) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetOutputStreamAttributes(DWORD dwOutputStreamID, IMFAttributes **ppAttributes) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE DeleteInputStream(DWORD dwStreamID) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE AddInputStreams(DWORD cStreams, DWORD* adwStreamIDs) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetInputAvailableType(DWORD dwInputStreamID, DWORD dwTypeIndex, IMFMediaType** ppType) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE SetInputType(DWORD dwInputStreamID, IMFMediaType* pType, DWORD dwFlags) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE SetOutputType(DWORD dwOutputStreamID, IMFMediaType* pType, DWORD dwFlags) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetInputCurrentType(DWORD dwInputStreamID, IMFMediaType** ppType) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetOutputCurrentType(DWORD dwOutputStreamID, IMFMediaType** ppType) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetInputStatus(DWORD dwInputStreamID, DWORD* pdwFlags) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE GetOutputStatus(DWORD* pdwFlags) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE SetOutputBounds(LONGLONG hnsLowerBound, LONGLONG hnsUpperBound) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE ProcessEvent(DWORD dwInputStreamID, IMFMediaEvent* pEvent) override;
	UE_API virtual HRESULT STDMETHODCALLTYPE ProcessInput(DWORD dwInputStreamID, IMFSample* pSample, DWORD dwFlags) override;

	UE_API virtual HRESULT OnSetOutputType(IMFMediaType *InMediaType);
	UE_API virtual HRESULT OnFlush();
	UE_API virtual HRESULT OnDiscontinuity();

	/**
	 * See if this decoder supports external buffers.
	 *
	 * @return True if so.
	 */
	UE_API virtual bool IsExternalBufferSupported() const;

	/**
	 * Call this to use our external buffers and not Windows.
	 * 
	 * @param bInEnable True to use external.
	 */
	UE_API void EnableExternalBuffer(bool bInEnable);

	/**
	 * See if external buffers are enabled.
	 *
	 * @return True if so.
	 */
	bool IsExternalBufferEnabled() const { return bIsExternalBufferEnabled; }

	/**
	 * Call this to get a decoded external buffer from the decoder.
	 * InBuffer does not need to have any space.
	 * This will move the data to InBuffer.
	 * 
	 * @return True if a buffer was found.
	 * @param InBuffer This will be updated with the data of the buffer.
	 * @param TimeStamp TimeStamp of the requested buffer.
	 */
	UE_API bool GetExternalBuffer(TArray<uint8>& InBuffer, uint64 TimeStamp);

	/**
	 * Call this to return an external buffer to the decoder.
	 * The data in InBuffer will be moved out.
	 *
	 * @param InBuffer Buffer to return.
	 */
	static UE_API void ReturnExternalBuffer(TArray<uint8>& InBuffer);

protected:

	/**
	 * Call this to get an external buffer to decode into.
	 * 
	 * @param InTimeStamp TimeStamp for the buffer.
	 * @param InSize Desired size of this buffer.
	 */
	UE_API TArray<uint8>* AllocateExternalBuffer(uint64 InTimeStamp, int32 InSize);
	
private:

	virtual HRESULT OnCheckInputType(IMFMediaType *InMediaType) = 0;
	virtual HRESULT OnSetInputType(IMFMediaType* InMediaType) = 0;
	virtual bool HasPendingOutput() const = 0;
	virtual HRESULT InternalProcessInput(LONGLONG InTimeStamp, BYTE* InData, DWORD InDataSize) = 0;

	UE_API void EmplyQueues();
	UE_API HRESULT OnCheckOutputType(IMFMediaType *InMediaType);

	/**
	 * Call this to return an external buffer to the decoder.
	 * 
	 * @param InBuffer Buffer to return.
	 */
	UE_API void ReturnExternalBufferInternal(TArray<uint8>& InBuffer);

	/**
	 * Removes this decoder from the map from GetMapBufferToDecoder.
	 */
	UE_API void RemoveDecoderFromMap();

	/**
	 * Access to the map from GetMapBufferToDecoder should be guarded with this.
	 */
	static UE_API FCriticalSection& GetMapBufferCriticalSection();

	/**
	 * Returns a map that lets you find which decoder a buffer is using.
	 * Lock GetMapBufferCriticalSection before using this.
	 */
	static UE_API TMap<uint8*, WmfMediaDecoder*>& GetMapBufferToDecoder();

protected:

	int32 RefCount;

	FCriticalSection CriticalSection;

	TComPtr<IMFMediaType> InputType;
	TComPtr<IMFMediaType> OutputType;

	UINT32 ImageWidthInPixels;
	UINT32 ImageHeightInPixels;
	MFRatio FrameRate;
	DWORD InputImageSize;
	DWORD OutputImageSize;

	TComPtr<IMFDXGIDeviceManager> DXGIManager;
	TComPtr<ID3D11Device> D3D11Device;
	TComPtr<ID3D11DeviceContext> D3DImmediateContext;

	LONGLONG InternalTimeStamp;
	UINT64 SampleDuration;

	TQueue<DataBuffer> InputQueue;
	TQueue<DataBuffer> OutputQueue;

	/** True if we are using our external buffers and not Windows. */
	bool bIsExternalBufferEnabled;
	/** Maps a time stamp to a buffer. */
	TMap<uint64, TArray<uint8>*> MapTimeStampToExternalBuffer;
	/** Holds all the buffers that are not being used. */
	TArray<TArray<uint8>*> ExternalBufferPool;
	/** Guards access to the buffers. */
	FCriticalSection BufferCriticalSection;

};

#endif // WMFMEDIA_SUPPORTED_PLATFORM

#undef UE_API
