// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "RenderUtils.h"
#include "RHIStaticStates.h"
#include "PipelineStateCache.h"
#include "ShaderParameterUtils.h"
#include "MediaShaders.h"
#include "DataDrivenShaderPlatformInfo.h"

BEGIN_SHADER_PARAMETER_STRUCT(FHardwareVideoDecodingShaderParams, )
	SHADER_PARAMETER_SRV(Texture2D, TextureY)
	SHADER_PARAMETER_SRV(Texture2D, TextureUV)

	SHADER_PARAMETER_SAMPLER(SamplerState, PointClampedSamplerY)
	SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampedSamplerUV)
	SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampedSamplerUVAlpha)

	SHADER_PARAMETER(FMatrix44f, ColorTransform)
	SHADER_PARAMETER(uint32, SrgbToLinear)
END_SHADER_PARAMETER_STRUCT()

/**
 * Shaders which allow the conversion of NV12 texture data into RGB textures.
 */
class FWmfMediaHardwareVideoDecodingShader : public FGlobalShader
{
public:
	using FParameters = FHardwareVideoDecodingShaderParams;

	FWmfMediaHardwareVideoDecodingShader() = default;
	FWmfMediaHardwareVideoDecodingShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			|| (IsPCPlatform(Parameters.Platform) && !IsOpenGLPlatform(Parameters.Platform)); // to support mobile emulation on PC
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static FHardwareVideoDecodingShaderParams GetCommonParameters(bool InIsOutputSrgb)
	{
		FHardwareVideoDecodingShaderParams Parameters{};

		Parameters.PointClampedSamplerY = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		Parameters.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(MediaShaders::YuvToRgbRec709Scaled, MediaShaders::YUVOffset8bits);

		// Explicitly specify integer value, as using boolean falls over on some platforms.
		Parameters.SrgbToLinear = InIsOutputSrgb ? 1 : 0;
		return Parameters;
	}

	static FHardwareVideoDecodingShaderParams GetParameters(FRHIShaderResourceView* InTextureY, FRHIShaderResourceView* InTextureUV, bool InIsOutputSrgb, bool InFilterUV = true)
	{
		FHardwareVideoDecodingShaderParams Parameters = GetCommonParameters(InIsOutputSrgb);

		Parameters.TextureY = InTextureY;
		Parameters.TextureUV = InTextureUV;

		if (InFilterUV)
		{
			Parameters.BilinearClampedSamplerUV = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters.BilinearClampedSamplerUVAlpha = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
		else
		{
			Parameters.BilinearClampedSamplerUV = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters.BilinearClampedSamplerUVAlpha = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
		return Parameters;
	}

	static FHardwareVideoDecodingShaderParams GetParameters(FRHIShaderResourceView* InTextureRGBA, bool InIsOutputSrgb)
	{
		FHardwareVideoDecodingShaderParams Parameters = GetCommonParameters(InIsOutputSrgb);
		Parameters.TextureY = InTextureRGBA;
		Parameters.BilinearClampedSamplerUV = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		return Parameters;
	}
};

class FHardwareVideoDecodingVS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingVS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingVS, FWmfMediaHardwareVideoDecodingShader);
};

class FHardwareVideoDecodingPS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingPS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingPS, FWmfMediaHardwareVideoDecodingShader);
};

class FHardwareVideoDecodingPassThroughPS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingPassThroughPS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingPassThroughPS, FWmfMediaHardwareVideoDecodingShader);
};

class FHardwareVideoDecodingY416PS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingY416PS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingY416PS, FWmfMediaHardwareVideoDecodingShader);
};


class FHardwareVideoDecodingYCoCgPS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingYCoCgPS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingYCoCgPS, FWmfMediaHardwareVideoDecodingShader);
};


class FHardwareVideoDecodingYCoCgAlphaPS : public FWmfMediaHardwareVideoDecodingShader
{
public:
	DECLARE_GLOBAL_SHADER(FHardwareVideoDecodingYCoCgAlphaPS);
	SHADER_USE_PARAMETER_STRUCT(FHardwareVideoDecodingYCoCgAlphaPS, FWmfMediaHardwareVideoDecodingShader);
};


