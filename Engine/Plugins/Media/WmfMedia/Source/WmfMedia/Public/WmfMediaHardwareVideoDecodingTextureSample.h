// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_WINDOWS

#include "Player/WmfMediaTextureSample.h"
#include "RHI.h"
#include "RHICommandList.h"

#include "IMediaTextureSampleConverter.h"
#include "WmfMediaHardwareVideoDecodingRendering.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d11.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "RenderUtils.h"

#include "Microsoft/COMPointer.h"

/**
 * Texture sample for hardware video decoding.
 */
class FWmfMediaHardwareVideoDecodingTextureSample :
	public FWmfMediaTextureSample, 
	public IMediaTextureSampleConverter
{
public:

	/** Default constructor. */
	FWmfMediaHardwareVideoDecodingTextureSample()
		: FWmfMediaTextureSample()
		, Format(PF_Unknown)
		, AlphaFormat(PF_Unknown)
		, bIsDestinationTextureSRGB(false)
	{ }

public:

	/**
	 * Initialize shared texture from Wmf device
	 *
	 * @param InD3D11Device WMF device to create shared texture from
	 * @param InTime The sample time (in the player's local clock).
	 * @param InDuration The sample duration
	 * @param InDim texture dimension to create
	 * @param InFormat texture format to create
	 * @param InMediaTextureSampleFormat Media texture sample format
	 * @param InCreateFlags texture create flag
	 * @return The texture resource object that will hold the sample data.
	 */
	WMFMEDIA_API ID3D11Texture2D* InitializeSourceTexture(const TRefCountPtr<ID3D11Device>& InD3D11Device, FMediaTimeStamp InTime, FTimespan InDuration, const FIntPoint& InDim, EPixelFormat InFormat, EMediaTextureSampleFormat InMediaTextureSampleFormat);

	/**
	 * Call this to set the pixel format.
	 * 
	 * InitializeSourceTexture does this so you only need to call this function if
	 * you don't call InitializeSourceTexture.
	 * 
	 * @param InFormat Pixel format for texture.
	 */
	void SetPixelFormat(EPixelFormat InFormat) { Format = InFormat; }

	/**
	 * Call this to set whether DestinationTexture should be sRGB or not.
	 * 
	 * @param bIsSRGB True if so.
	 */
	void SetIsDestinationTextureSRGB(bool bIsSRGB) { bIsDestinationTextureSRGB = bIsSRGB; }

	/**
	 * Call this to set what the alpha texture should be.
	 * 
	 * @param InFormat Pixel format of alpha texture.
	 */
	void SetAlphaTexture(EPixelFormat InFormat) { AlphaFormat = InFormat; }

	/**
	 * Get media texture sample converter if sample implements it
	 *
	 * @return texture sample converter
	 */
	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override
	{
		// Only use sample converter for Win8+
		if (FPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			return (SourceTexture.IsValid() || DestinationTexture.IsValid() || IsBufferExternal()) ? this : nullptr;
		}
		return nullptr;
	}

	/**
	 * Texture sample convert using hardware video decoding.
	 */
	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override
	{
		FWmfMediaHardwareVideoDecodingParameters::ConvertTextureFormat_RenderThread(RHICmdList, this, InDstTexture);
		return true;
	}

	/**
	 * Get source texture from WMF device
	 *
	 * @return Source texture
	 */
	TComPtr<ID3D11Texture2D> GetSourceTexture() const
	{
		return SourceTexture;
	}

	/**
	 * Get Destination Texture of render thread device
	 *
	* @param RHICmdList the command list used to create the texture.
	 * @return Destination texture 
	 */
	FTextureRHIRef GetOrCreateDestinationTexture(FRHICommandListBase& RHICmdList)
	{
		if (DestinationTexture.IsValid() && DestinationTexture->GetSizeX() == Dim.X && DestinationTexture->GetSizeY() == Dim.Y)
		{
			ETextureCreateFlags CurrentFlags = DestinationTexture->GetFlags();
			bool bIsCurrentSRGB = EnumHasAnyFlags(CurrentFlags, ETextureCreateFlags::SRGB);
			if (bIsCurrentSRGB == bIsDestinationTextureSRGB)
			{
				return DestinationTexture;
			}
		}

		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FWmfMediaHardwareVideoDecodingTextureSample_DestinationTexture"), Dim, Format)
			.SetFlags(ETextureCreateFlags::DisableSRVCreation);

		if (bIsDestinationTextureSRGB)
		{
			Desc.AddFlags(ETextureCreateFlags::SRGB);
		}

		DestinationTexture = RHICmdList.CreateTexture(Desc);

		return DestinationTexture;
	}

	/**
	 * Get Destination Texture of render thread device
	 *
	* @param RHICmdList the command list used to create the texture.
	 * @return Destination texture
	 */
	FTextureRHIRef GetOrCreateDestinationAlphaTexture(FRHICommandListBase& RHICmdList)
	{
		if (DestinationAlphaTexture.IsValid() && DestinationAlphaTexture->GetSizeX() == Dim.X && DestinationAlphaTexture->GetSizeY() == Dim.Y)
		{
			return DestinationAlphaTexture;
		}

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FWmfMediaHardwareVideoDecodingTextureSample_DestinationAlphaTexture"), Dim, AlphaFormat)
			.SetFlags(ETextureCreateFlags::DisableSRVCreation);
		
		DestinationAlphaTexture = RHICmdList.CreateTexture(Desc);

		return DestinationAlphaTexture;
	}


	/**
	 * Called the the sample is returned to the pool for cleanup purposes
	 */
#if !UE_SERVER
	WMFMEDIA_API virtual void ShutdownPoolable() override;
#endif

private:

	/** Source Texture resource (from Wmf device). */
	TComPtr<ID3D11Texture2D> SourceTexture;

	/** D3D11 Device which create the texture, used to release the keyed mutex when the sampled is returned to the pool */
	TRefCountPtr<ID3D11Device> D3D11Device;

	/** Destination Texture resource (from Rendering device) */
	FTextureRHIRef DestinationTexture;

	/** Destination Texture resource (from Rendering device) */
	FTextureRHIRef DestinationAlphaTexture;

	/** Texture format */
	EPixelFormat Format;

	/** Texture format */
	EPixelFormat AlphaFormat;

	/** Whether DestinationTexture should be an sRGB texture or not. */
	bool bIsDestinationTextureSRGB;
};

/** Implements a pool for WMF texture samples. */
class FWmfMediaHardwareVideoDecodingTextureSamplePool : public TMediaObjectPool<FWmfMediaHardwareVideoDecodingTextureSample> { };

#endif
