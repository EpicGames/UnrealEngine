// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WmfMedia : ModuleRules
	{
		public WmfMedia(ReadOnlyTargetRules Target) : base(Target)
		{
            bEnableExceptions = true;
			bDisableAutoRTFMInstrumentation = true; // AutoRTFM cannot be used with exceptions

            DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "WmfMediaFactory"
                    });

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
                    "Engine",
                    "MediaUtils",
					"Projects",
					"RenderCore",
                    "RHI",
                });


            PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.Add("Engine");
				PrivateDependencyModuleNames.Add("HeadMountedDisplay");
			}

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PrivateDependencyModuleNames.Add("D3D11RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");

				PublicDelayLoadDLLs.Add("mf.dll");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicDelayLoadDLLs.Add("mfplay.dll");
				PublicDelayLoadDLLs.Add("shlwapi.dll");
			}
		}
	}
}
