// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfFileMediaSourceFactory.h"

#include "FileMediaSource.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WmfFileMediaSourceFactory)


/* UWmfFileMediaSourceFactory structors
 *****************************************************************************/

UWmfFileMediaSourceFactory::UWmfFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("3g2;3G2 Multimedia Stream"));
	Formats.Add(TEXT("3gp;3GP Video Stream"));
	Formats.Add(TEXT("3gp2;3GPP2 Multimedia File"));
	Formats.Add(TEXT("3gpp;3GPP Multimedia File"));
	Formats.Add(TEXT("aac;MPEG-2 Advanced Audio Coding File"));
	Formats.Add(TEXT("adts;Audio Data Transport Stream"));
	Formats.Add(TEXT("asf;ASF Media File"));
	Formats.Add(TEXT("avi;Audio Video Interleave File"));
	Formats.Add(TEXT("m4a;Apple MPEG-4 Audio"));
	Formats.Add(TEXT("m4v;Apple MPEG-4 Video"));
	Formats.Add(TEXT("mov;Apple QuickTime Movie"));
	Formats.Add(TEXT("mp4;MPEG-4 Movie"));
	Formats.Add(TEXT("sami;Synchronized Accessible Media Interchange (SAMI) File"));
	Formats.Add(TEXT("smi;Synchronized Multimedia Integration (SMIL) File"));
	Formats.Add(TEXT("wav;Wave Audio File"));
	Formats.Add(TEXT("wma;Windows Media Audio"));
	Formats.Add(TEXT("wmv;Windows Media Video"));

	SupportedClass = UFileMediaSource::StaticClass();
	bEditorImport = true;
}


/* UFactory overrides
 *****************************************************************************/

bool UWmfFileMediaSourceFactory::FactoryCanImport(const FString& Filename)
{
	// @hack: disable file extensions that are used in other factories
	// @todo gmp: add support for multiple factories per file extension
	const FString FileExtension = FPaths::GetExtension(Filename);

	return (FileExtension.ToUpper() != FString("WAV"));
}


UObject* UWmfFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}

