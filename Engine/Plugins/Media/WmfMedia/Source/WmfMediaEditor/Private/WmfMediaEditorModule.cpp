// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"


/**
 * Implements the WmfMediaEditor module.
 */
class FWmfMediaEditorModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FWmfMediaEditorModule, WmfMediaEditor);
