// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaFactoryPrivate.h"

#include "IMediaModule.h"
#include "IMediaOptions.h"
#include "IMediaPlayerFactory.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "WmfMediaSettings.h"
#else
	#include "Internationalization/Internationalization.h"
	#include "Misc/Guid.h"
#endif

#include "IWmfMediaModule.h"


DEFINE_LOG_CATEGORY(LogWmfMediaFactory);

#define LOCTEXT_NAMESPACE "FWmfMediaFactoryModule"


/**
 * Implements the WmfMediaFactory module.
 */
class FWmfMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	/** Default constructor. */
	FWmfMediaFactoryModule() { }

public:

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		// check options
		if ((OutWarnings != nullptr) && (Options != nullptr))
		{
			if (Options->GetMediaOption("PrecacheFile", false) && (Scheme != TEXT("file")))
			{
				OutWarnings->Add(LOCTEXT("PrecachingNotSupported", "Precaching is supported for local files only"));
			}
		}

		return 80;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto WmfMediaModule = FModuleManager::LoadModulePtr<IWmfMediaModule>("WmfMedia");
		return (WmfMediaModule != nullptr) ? WmfMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Windows Media Foundation");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("WmfMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x82f2a4c0, 0x225448c6, 0x853ff0a3, 0x0a2b08b7);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioSamples) ||
				(Feature == EMediaFeature::AudioTracks) ||
				(Feature == EMediaFeature::CaptionTracks) ||
				(Feature == EMediaFeature::MetadataTracks) ||
				(Feature == EMediaFeature::OverlaySamples) ||
				(Feature == EMediaFeature::SubtitleTracks) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported file extensions
		SupportedFileExtensions.Add(TEXT("3g2"));
		SupportedFileExtensions.Add(TEXT("3gp"));
		SupportedFileExtensions.Add(TEXT("3gp2"));
		SupportedFileExtensions.Add(TEXT("3gpp"));
		SupportedFileExtensions.Add(TEXT("aac"));
		SupportedFileExtensions.Add(TEXT("adts"));
		SupportedFileExtensions.Add(TEXT("asf"));
		SupportedFileExtensions.Add(TEXT("avi"));
		SupportedFileExtensions.Add(TEXT("m2ts"));
		SupportedFileExtensions.Add(TEXT("m4a"));
		SupportedFileExtensions.Add(TEXT("m4v"));
		SupportedFileExtensions.Add(TEXT("mov"));
		SupportedFileExtensions.Add(TEXT("mp4"));
		SupportedFileExtensions.Add(TEXT("sami"));
		SupportedFileExtensions.Add(TEXT("smi"));
		SupportedFileExtensions.Add(TEXT("wav"));
		SupportedFileExtensions.Add(TEXT("wma"));
		SupportedFileExtensions.Add(TEXT("wmv"));

		// supported platforms
		SupportedPlatforms.Add(TEXT("Windows"));
		AddSupportedPlatform(FGuid(0x5636fbc1, 0xd2b54f62, 0xac8e7d4f, 0xb184b45a));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("audcap"));
		SupportedUriSchemes.Add(TEXT("file"));
		SupportedUriSchemes.Add(TEXT("http"));
		SupportedUriSchemes.Add(TEXT("httpd"));
		SupportedUriSchemes.Add(TEXT("https"));
		SupportedUriSchemes.Add(TEXT("mms"));
		SupportedUriSchemes.Add(TEXT("rtsp"));
		SupportedUriSchemes.Add(TEXT("rtspt"));
		SupportedUriSchemes.Add(TEXT("rtspu"));
		SupportedUriSchemes.Add(TEXT("vidcap"));

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "WmfMedia",
				LOCTEXT("WmfMediaSettingsName", "WMF Media"),
				LOCTEXT("WmfMediaSettingsDescription", "Configure the WMF Media plug-in."),
				GetMutableDefault<UWmfMediaSettings>()
			);
		}
#endif //WITH_EDITOR

		// register player factory
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}

#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "WmfMedia");
		}
#endif //WITH_EDITOR
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule)
		{
			FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
			if (!PlatformName.IsNone())
			{
				SupportedPlatforms.Add(PlatformName.ToString());
			}
		}
	}

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWmfMediaFactoryModule, WmfMediaFactory);
