// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

/** Log category for the WmfMediaFactory module. */
DECLARE_LOG_CATEGORY_EXTERN(LogWmfMediaFactory, Log, All);
