// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WmfMediaSettings)

UWmfMediaSettings::UWmfMediaSettings()
	: AllowNonStandardCodecs(false)
	, LowLatency(false)
	, NativeAudioOut(false)
	, HardwareAcceleratedVideoDecoding(false)
	, bAreHardwareAcceleratedCodecRegistered(false)
{ }

void UWmfMediaSettings::EnableHardwareAcceleratedCodecRegistered()
{
	bAreHardwareAcceleratedCodecRegistered = true;
}

