// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "WmfMediaSettings.generated.h"

#define UE_API WMFMEDIAFACTORY_API


/**
 * Settings for the WmfMedia plug-in.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UWmfMediaSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UE_API UWmfMediaSettings();

public:

	/**
	 * Whether to allow the loading of media that uses non-standard codecs (default = off).
	 *
	 * By default, the player will attempt to detect audio and video codecs that
	 * are not supported by the operating system out of the box, but may require
	 * the user to install additional codec packs. Enable this option to skip
	 * this check and allow the usage of non-standard codecs.
	 */
	UPROPERTY(config, EditAnywhere, Category=Media)
	bool AllowNonStandardCodecs;

	/**
	 * Enable low latency processing in the Windows media pipeline (default = off).
	 *
	 * When this setting is enabled, the media data is generated with the lowest
	 * possible delay. This might be desirable for certain real-time applications,
	 * but it can negatively affect audio and video quality.
	 *
	 * @note This setting is only supported on Windows 8 or newer
	 */
	UPROPERTY(config, EditAnywhere, Category=Media)
	bool LowLatency;

	/** Play audio tracks via the operating system's native sound mixer (default = off). */
	UPROPERTY(config, EditAnywhere, Category=Debug)
	bool NativeAudioOut;

	/** Use hardware accelerated video acceleration (GPU) when possible otherwise fallback to software implementation (CPU), Windows and DX11 only. */
	UPROPERTY(config, EditAnywhere, Category=Media, meta = (DisplayName = "Hardware Accelerated Video Decoding (Experimental)"))
	bool HardwareAcceleratedVideoDecoding;

	/** Set when at least one registered codec is hardware accelerated. */
	bool bAreHardwareAcceleratedCodecRegistered;

public:

	UE_API void EnableHardwareAcceleratedCodecRegistered();

};

#undef UE_API
