// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WmfMediaFactory : ModuleRules
	{
		public WmfMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
					"WmfMedia",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			if (Target.Type == TargetType.Editor)
			{
				DynamicallyLoadedModuleNames.Add("Settings");
				PrivateIncludePathModuleNames.Add("Settings");
			}

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				DynamicallyLoadedModuleNames.Add("WmfMedia");
			}
		}
	}
}
