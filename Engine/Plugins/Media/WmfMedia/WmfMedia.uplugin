{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "WMF Media Player",
	"Description" : "Implements a media player using the Windows Media Foundation framework.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "WmfMedia",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "WmfMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "WmfMediaFactory",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "WmfMediaFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Mac", "Linux", "Win64" ],
			"TargetDenyList": [ "Server" ]
		}
	]
}
